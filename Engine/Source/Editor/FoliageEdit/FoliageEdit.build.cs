// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class FoliageEdit : ModuleRules
{
	public FoliageEdit(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] 
			{
				"Core",
				"CoreUObject",
				"InputCore",
				"Engine",
				"EditorFramework",
				"EditorSubsystem",
				"UnrealEd",
				"Slate",
				"SlateCore",
				"RenderCore",
				"LevelEditor",
				"SceneOutliner",
				"Landscape",
                "PropertyEditor",
                "DetailCustomizations",
                "AssetTools",
                "Foliage",
				"DataLayerEditor",
				"EditorWidgets",
				"ToolWidgets",
			}
		);

	}
}
