// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
ActorFactory.cpp:
=============================================================================*/

#include "ActorFactoryProceduralFoliage.h"

#include "ActorFactories/ActorFactory.h"
#include "AssetRegistry/AssetData.h"
#include "Containers/UnrealString.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "ProceduralFoliageComponent.h"
#include "ProceduralFoliageSpawner.h"
#include "ProceduralFoliageVolume.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFactoryProceduralFoliage)

#define LOCTEXT_NAMESPACE "ActorFactoryProceduralFoliage"

/*-----------------------------------------------------------------------------
UActorFactoryProceduralFoliage
-----------------------------------------------------------------------------*/
UActorFactoryProceduralFoliage::UActorFactoryProceduralFoliage(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("ProceduralFoliageDisplayName", "Procedural Foliage Volume");
	NewActorClass = AProceduralFoliageVolume::StaticClass();
	bUseSurfaceOrientation = true;
}

bool UActorFactoryProceduralFoliage::PreSpawnActor(UObject* Asset, FTransform& InOutLocation)
{
	return GetDefault<UEditorExperimentalSettings>()->bProceduralFoliage;
}

bool UActorFactoryProceduralFoliage::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!AssetData.IsValid() || !AssetData.IsInstanceOf(UProceduralFoliageSpawner::StaticClass()))
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoProceduralFoliageSpawner", "A valid ProceduralFoliageSpawner must be specified.");
		return false;
	}

	return true;
}

void UActorFactoryProceduralFoliage::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);
	UProceduralFoliageSpawner* FoliageSpawner = CastChecked<UProceduralFoliageSpawner>(Asset);

	UE_LOGF(LogActorFactory, Log, "Actor Factory created %ls", *FoliageSpawner->GetName());

	// Change properties
	AProceduralFoliageVolume* PFV = CastChecked<AProceduralFoliageVolume>(NewActor);
	UProceduralFoliageComponent* ProceduralComponent = PFV->ProceduralComponent;
	check(ProceduralComponent);

	ProceduralComponent->UnregisterComponent();

	ProceduralComponent->FoliageSpawner = FoliageSpawner;

	// Init Component
	ProceduralComponent->RegisterComponent();
}

UObject* UActorFactoryProceduralFoliage::GetAssetFromActorInstance(AActor* Instance)
{
	check(Instance->IsA(NewActorClass));

	AProceduralFoliageVolume* PFV = CastChecked<AProceduralFoliageVolume>(Instance);
	UProceduralFoliageComponent* ProceduralComponent = PFV->ProceduralComponent;
	check(ProceduralComponent);
	
	return ProceduralComponent->FoliageSpawner;
}
#undef LOCTEXT_NAMESPACE
