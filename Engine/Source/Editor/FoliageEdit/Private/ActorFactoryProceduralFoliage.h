// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "ActorFactories/ActorFactoryBoxVolume.h"
#include "Math/Transform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ActorFactoryProceduralFoliage.generated.h"

class AActor;
class FText;
class UObject;
struct FAssetData;

UCLASS(MinimalAPI, config=Editor)
class UActorFactoryProceduralFoliage : public UActorFactoryBoxVolume
{
	GENERATED_UCLASS_BODY()

	//~ Begin UActorFactory Interface
	virtual bool PreSpawnActor(UObject* Asset, FTransform& InOutLocation);
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;
	virtual void PostSpawnActor( UObject* Asset, AActor* NewActor) override;
	virtual UObject* GetAssetFromActorInstance(AActor* ActorInstance) override;
	//~ End UActorFactory Interface
};
