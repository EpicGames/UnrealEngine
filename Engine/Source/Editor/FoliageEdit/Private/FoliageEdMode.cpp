// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageEdMode.h"

#include "ActorPartition/ActorPartitionSubsystem.h"
#include "ActorPartition/PartitionActor.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "IAssetTools.h"
#include "CollisionQueryParams.h"
#include "CollisionShape.h"
#include "Components/ActorComponent.h"
#include "Components/BrushComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/ModelComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SceneComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "Engine/Blueprint.h"
#include "Engine/Brush.h"
#include "Engine/CollisionProfile.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/HitResult.h"
#include "Engine/Level.h"
#include "Engine/NetSerialization.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/World.h"
#include "EngineDefines.h"
#include "EngineUtils.h"
#include "FoliageEdModeToolkit.h"
#include "FoliageEditActions.h"
#include "FoliageEditUtility.h"
#include "FoliageHelper.h"
#include "FoliageInstanceBase.h"
#include "FoliageInstancedStaticMeshComponent.h"
#include "FoliageType.h"
#include "FoliageType_Actor.h"
#include "FoliageType_InstancedStaticMesh.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameFramework/Actor.h"
#include "HAL/IConsoleManager.h"
#include "HitProxies.h"
//Slate dependencies
#include "IAssetViewport.h"
#include "ILevelEditor.h"
#include "ISceneOutliner.h"
#include "InstancedFoliageActor.h"
#include "Instances/InstancedPlacementHash.h"
#include "Internationalization/Internationalization.h"
#include "LandscapeComponent.h"
#include "LandscapeHeightfieldCollisionComponent.h"
// Classes
#include "LandscapeInfo.h"
#include "LevelEditor.h"
#include "LevelUtils.h"
#include "Logging/LogMacros.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Math/Interval.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Guid.h"
#include "Misc/ScopeExit.h"
#include "Model.h"
#include "Modules/ModuleManager.h"
#include "RawIndexBuffer.h"
#include "Rendering/ColorVertexBuffer.h"
#include "Rendering/PositionVertexBuffer.h"
#include "Rendering/StaticMeshVertexBuffer.h"
#include "SceneView.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "Serialization/Archive.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"
#include "Stats/Stats.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Templates/SubclassOf.h"
#include "Templates/Tuple.h"
#include "Templates/TypeHash.h"
#include "Templates/UniqueObj.h"
#include "Templates/UnrealTemplate.h"
#include "Toolkits/BaseToolkit.h"
#include "Toolkits/ToolkitManager.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UnrealClient.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "WorldPartition/WorldPartitionSubsystem.h"

class UClass;

#define LOCTEXT_NAMESPACE "FoliageEdMode"
#define FOLIAGE_SNAP_TRACE (10000.f)

DEFINE_LOG_CATEGORY_STATIC(LogFoliage, Log, Log);

DECLARE_CYCLE_STAT(TEXT("Calculate Potential Instance"), STAT_FoliageCalculatePotentialInstance, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Add Instance Imp"), STAT_FoliageAddInstanceImp, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Add Instance For Brush"), STAT_FoliageAddInstanceBrush, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Remove Instance For Brush"), STAT_FoliageRemoveInstanceBrush, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Spawn Instance"), STAT_FoliageSpawnInstance, STATGROUP_Foliage);

namespace VREd
{
	static FAutoConsoleVariable FoliageOpacity(TEXT("VREd.FoliageOpacity"), 0.02f, TEXT("The foliage brush opacity."));
}

class FEdModeFoliageSelectionUpdate
{
public:
	FEdModeFoliageSelectionUpdate(FEdModeFoliage* InMode)
		: Mode(InMode)
	{
		Mode->BeginSelectionUpdate();
	}

	~FEdModeFoliageSelectionUpdate()
	{
		Mode->EndSelectionUpdate();
	}

private:
	FEdModeFoliage* Mode;
};

//
// FFoliageMeshUIInfo
//
FFoliageMeshUIInfo::FFoliageMeshUIInfo(UFoliageType* InSettings)
	: Settings(InSettings)
	, InstanceCountCurrentLevel(0)
	, InstanceCountTotal(0)
{
}

FText FFoliageMeshUIInfo::GetNameText() const
{
	//@todo: this is redundant with FFoliagePaletteItem::DisplayFName, should probably move sorting implementation over to SFoliagePalette
	FName DisplayFName = Settings->GetDisplayFName();
	return FText::FromName(DisplayFName);
}

//
// FFoliageInfo iterator
//
class FFoliageInfoIterator
{
private:
	const UFoliageType* FoliageType;
	FFoliageInfo* CurrentInfo;
	TActorIterator<AInstancedFoliageActor> ActorIterator;
	AInstancedFoliageActor* CurrentIFA;

public:
	FFoliageInfoIterator(UWorld* InWorld, const UFoliageType* InFoliageType)
		: FoliageType(InFoliageType)
		, CurrentInfo(nullptr)
		, ActorIterator(InWorld)
		, CurrentIFA(nullptr)
	{
		// shortcut for non-assets
		if (!InFoliageType->IsAsset())
		{
			AInstancedFoliageActor* IFA = Cast<AInstancedFoliageActor>(FoliageType->GetOuter());
			if (IFA->GetLevel()->bIsVisible)
			{
				CurrentIFA = IFA;
				CurrentInfo = CurrentIFA->FindInfo(FoliageType);
			}
		}
		else
		{
			UpdateCurrent();
		}
	}

	void operator++()
	{
		// Stop iteration
		if (!FoliageType->IsAsset() || !ActorIterator)
		{
			CurrentIFA = nullptr;
			CurrentInfo = nullptr;
			return;
		}

		++ActorIterator;
		UpdateCurrent();
	}

	FORCEINLINE FFoliageInfo* operator*()
	{
		check(CurrentInfo);
		return CurrentInfo;
	}

	FORCEINLINE explicit operator bool() const
	{
		return CurrentInfo != nullptr;
	}

	FORCEINLINE AInstancedFoliageActor* GetActor()
	{
		return CurrentIFA;
	}

private:
	void UpdateCurrent()
	{
		while (ActorIterator)
		{
			CurrentIFA = *ActorIterator;
			CurrentInfo = CurrentIFA->FindInfo(FoliageType);
			if (CurrentInfo)
			{
				return;
			}
			++ActorIterator;
		}

		CurrentInfo = nullptr;
		CurrentIFA = nullptr;
	}
};

//
// Painting filtering options
//
bool FFoliagePaintingGeometryFilter::operator() (const UPrimitiveComponent* Component) const
{
	if (Component)
	{
		bool bFoliageOwned = Component->GetOwner() && FFoliageHelper::IsOwnedByFoliage(Component->GetOwner());

		// allow list
		bool bAllowed =
			(bAllowLandscape   && Component->IsA(ULandscapeHeightfieldCollisionComponent::StaticClass())) ||
			(bAllowStaticMesh  && Component->IsA(UStaticMeshComponent::StaticClass()) && !Component->IsA(UFoliageInstancedStaticMeshComponent::StaticClass()) && !bFoliageOwned) ||
			(bAllowBSP         && (Component->IsA(UBrushComponent::StaticClass()) || Component->IsA(UModelComponent::StaticClass()))) ||
			(bAllowFoliage     && (Component->IsA(UFoliageInstancedStaticMeshComponent::StaticClass()) || bFoliageOwned));

		// deny list
		bAllowed &=
			(bAllowTranslucent || !(Component->GetMaterial(0) && IsTranslucentBlendMode(*Component->GetMaterial(0))));

		return bAllowed;
	}

	return false;
}

//
// FEdModeFoliage
//

static FName FoliageBrushHighlightColorParamName("HighlightColor");

/** Constructor */
FEdModeFoliage::FEdModeFoliage()
	: FEdMode()
	, bToolActive(false)
	, bCanAltDrag(false)
	, FoliageMeshListSortMode(EColumnSortMode::Ascending)
	, UpdateSelectionCounter(0)
	, bHasDeferredSelectionNotification(false)
	, bMoving(false)
	, bTracking(false)
{
	// Load resources and construct brush component
	UStaticMesh* StaticMesh = nullptr;
	BrushDefaultHighlightColor = FColor::White;
	if (!IsRunningCommandlet())
	{
		UMaterial* BrushMaterial = LoadObject<UMaterial>(nullptr, TEXT("/Engine/EditorLandscapeResources/FoliageBrushSphereMaterial.FoliageBrushSphereMaterial"), nullptr, LOAD_None, nullptr);
		BrushMID = UMaterialInstanceDynamic::Create(BrushMaterial, GetTransientPackage());
		check(BrushMID != nullptr);
		FLinearColor DefaultColor;
		BrushMID->GetVectorParameterDefaultValue(FoliageBrushHighlightColorParamName, DefaultColor);
		BrushDefaultHighlightColor = DefaultColor.ToFColor(false);
		StaticMesh = LoadObject<UStaticMesh>(nullptr, TEXT("/Engine/EngineMeshes/Sphere.Sphere"), nullptr, LOAD_None, nullptr);
	}
	BrushCurrentHighlightColor = BrushDefaultHighlightColor;
	SphereBrushComponent = NewObject<UStaticMeshComponent>(GetTransientPackage(), TEXT("SphereBrushComponent"));
	SphereBrushComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	SphereBrushComponent->SetCollisionObjectType(ECC_WorldDynamic);
	SphereBrushComponent->SetStaticMesh(StaticMesh);
	SphereBrushComponent->SetMaterial(0, BrushMID);
	SphereBrushComponent->SetAbsolute(true, true, true);
	SphereBrushComponent->CastShadow = false;

	bBrushTraceValid = false;
	BrushLocation = FVector::ZeroVector;
	
	// Get the default opacity from the material.
	FName OpacityParamName("OpacityAmount");
	BrushMID->GetScalarParameterValue(OpacityParamName, DefaultBrushOpacity);

}

void FEdModeFoliage::BindCommands()
{
	const FFoliageEditCommands& Commands = FFoliageEditCommands::Get();

	UICommandList->MapAction(
		Commands.IncreaseBrushSize,
		FExecuteAction::CreateRaw(this, &FEdModeFoliage::AdjustBrushRadius, 1.f),
		FCanExecuteAction::CreateRaw(this, &FEdModeFoliage::CurrentToolUsesBrush));

	UICommandList->MapAction(
		Commands.DecreaseBrushSize,
		FExecuteAction::CreateRaw(this, &FEdModeFoliage::AdjustBrushRadius, -1.f),
		FCanExecuteAction::CreateRaw(this, &FEdModeFoliage::CurrentToolUsesBrush));

	UICommandList->MapAction(
		Commands.IncreasePaintDensity,
		FExecuteAction::CreateRaw(this, &FEdModeFoliage::AdjustPaintDensity, 1.f),
		FCanExecuteAction::CreateRaw(this, &FEdModeFoliage::CurrentToolUsesBrush));

	UICommandList->MapAction(
		Commands.DecreasePaintDensity,
		FExecuteAction::CreateRaw(this, &FEdModeFoliage::AdjustPaintDensity, -1.f),
		FCanExecuteAction::CreateRaw(this, &FEdModeFoliage::CurrentToolUsesBrush));

	UICommandList->MapAction(
		Commands.IncreaseUnpaintDensity,
		FExecuteAction::CreateRaw(this, &FEdModeFoliage::AdjustUnpaintDensity, 1.f),
		FCanExecuteAction::CreateRaw(this, &FEdModeFoliage::CurrentToolUsesBrush));

	UICommandList->MapAction(
		Commands.DecreaseUnpaintDensity,
		FExecuteAction::CreateRaw(this, &FEdModeFoliage::AdjustUnpaintDensity, -1.f),
		FCanExecuteAction::CreateRaw(this, &FEdModeFoliage::CurrentToolUsesBrush));

	UICommandList->MapAction(
		Commands.SetPaint,
		FExecuteAction::CreateRaw(this, &FEdModeFoliage::OnSetPaint),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]
	{
		return UISettings.GetPaintToolSelected() && !UISettings.GetIsInSingleInstantiationMode();
	}));

	UICommandList->MapAction(
		Commands.SetReapplySettings,
		FExecuteAction::CreateRaw(this, &FEdModeFoliage::OnSetReapplySettings),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]
	{
		return UISettings.GetReapplyToolSelected() && !UISettings.GetIsInSingleInstantiationMode();
	}));

	UICommandList->MapAction(
		Commands.SetSelect,
		FExecuteAction::CreateRaw(this, &FEdModeFoliage::OnSetSelectInstance),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]
	{
		return UISettings.GetSelectToolSelected();
	}));

	UICommandList->MapAction(
		Commands.SetLassoSelect,
		FExecuteAction::CreateRaw(this, &FEdModeFoliage::OnSetLasso),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]
	{
		return UISettings.GetLassoSelectToolSelected();
	}));

	UICommandList->MapAction(
		Commands.SetPaintBucket,
		FExecuteAction::CreateRaw(this, &FEdModeFoliage::OnSetPaintFill),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]
	{
		return UISettings.GetPaintBucketToolSelected();
	}));

	UICommandList->MapAction(
		Commands.ReflectSelectionInPalette,
		FExecuteAction::CreateSP(this, &FEdModeFoliage::OnReflectSelectionInPalette));
}

bool FEdModeFoliage::CurrentToolUsesBrush() const
{
	return UISettings.GetPaintToolSelected() || UISettings.GetReapplyToolSelected() || UISettings.GetLassoSelectToolSelected();
}

/** Destructor */
FEdModeFoliage::~FEdModeFoliage()
{
	// Save UI settings to config file
	UISettings.Save();
	FEditorDelegates::MapChange.RemoveAll(this);
}


/** FGCObject interface */
void FEdModeFoliage::AddReferencedObjects(FReferenceCollector& Collector)
{
	// Call parent implementation
	FEdMode::AddReferencedObjects(Collector);

	Collector.AddReferencedObject(SphereBrushComponent);

	for (FFoliageMeshUIInfoPtr MeshUIInfo : FoliageMeshList)
	{
		Collector.AddReferencedObject(MeshUIInfo->Settings);
	}
}

/** FEdMode: Called when the mode is entered */
void FEdModeFoliage::Enter()
{
	FEdMode::Enter();

	// register for any objects replaced
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FEdModeFoliage::OnObjectsReplaced);
	FEditorDelegates::EndPIE.AddRaw(this, &FEdModeFoliage::OnEndPIE);


	// Clear any selection in case the instanced foliage actor is selected
	GEditor->SelectNone(true, true);

	// Load UI settings from config file
	UISettings.Load();

	// Bind to editor callbacks
	FEditorDelegates::NewCurrentLevel.AddSP(this, &FEdModeFoliage::NotifyNewCurrentLevel);
	FWorldDelegates::LevelAddedToWorld.AddSP(this, &FEdModeFoliage::NotifyLevelAddedToWorld);
	FWorldDelegates::LevelRemovedFromWorld.AddSP(this, &FEdModeFoliage::NotifyLevelRemovedFromWorld);
	AInstancedFoliageActor::SelectionChanged.AddSP(this, &FEdModeFoliage::NotifyActorSelectionChanged);
	AInstancedFoliageActor::InstanceCountChanged.AddSP(this, &FEdModeFoliage::OnInstanceCountUpdated);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnAssetRemoved().AddSP(this, &FEdModeFoliage::NotifyAssetRemoved);

	// Force real-time viewports.  We'll back up the current viewport state so we can restore it when the
	// user exits this mode.
	const bool bWantRealTime = true;
	ForceRealTimeViewports(bWantRealTime);

	if (!Toolkit.IsValid())
	{
		Toolkit = MakeShareable(new FFoliageEdModeToolkit);
		Toolkit->Init(Owner->GetToolkitHost());

		UICommandList = Toolkit->GetToolkitCommands();
		BindCommands();
	}

	if (UISettings.GetSelectToolSelected() || UISettings.GetLassoSelectToolSelected())
	{
		ApplySelection(GetWorld(), true);
	}

	TArray<AInstancedFoliageActor*> InstanceFoliageActorList;

	// Subscribe to mesh changed events (for existing and future IFA's)
	UWorld* World = GetWorld();
	OnActorSpawnedHandle = World->AddOnActorSpawnedHandler(FOnActorSpawned::FDelegate::CreateRaw(this, &FEdModeFoliage::HandleOnActorSpawned));

	for (TActorIterator<AInstancedFoliageActor> It(World); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		IFA->OnFoliageTypeMeshChanged().AddSP(this, &FEdModeFoliage::HandleOnFoliageTypeMeshChanged);
		IFA->EnterEditMode();
	}

	// Update UI
	NotifyNewCurrentLevel();

	// Make sure the brush is visible.
	SphereBrushComponent->SetVisibility(true);
}

/** FEdMode: Called when the mode is exited */
void FEdModeFoliage::Exit()
{
	if (bToolActive)
	{
		EndFoliageBrushTrace();
	}
	else if (bTracking)
	{
		EndTracking();
	}

	ensureMsgf(!GEditor->IsTransactionActive(), TEXT("Transaction Active when exiting foliage edit mode: '%s'"), *GEditor->GetTransactionName().ToString());

	FToolkitManager::Get().CloseToolkit(Toolkit.ToSharedRef());
	Toolkit.Reset();

	// Remove delegates
	FEditorDelegates::NewCurrentLevel.RemoveAll(this);
	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);
	AInstancedFoliageActor::SelectionChanged.RemoveAll(this);
	AInstancedFoliageActor::InstanceCountChanged.RemoveAll(this);

	if (FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry")))
	{
		IAssetRegistry* AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnAssetRemoved().RemoveAll(this);
		}
	}

	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);


	// Remove the brush
	SphereBrushComponent->UnregisterComponent();

	// Restore real-time viewport state if we changed it
	ForceRealTimeViewports(false);

	// Clear the cache (safety, should be empty!)
	LandscapeLayerCaches.Empty();

	// Save UI settings to config file
	UISettings.Save();

	// Clear selection visualization on any foliage items
	ApplySelection(GetWorld(), false);

	// Remove all event subscriptions
	UWorld* World = GetWorld();
	World->RemoveOnActorSpawnedHandler(OnActorSpawnedHandle);
	for (TActorIterator<AInstancedFoliageActor> It(World); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		IFA->OnFoliageTypeMeshChanged().RemoveAll(this);
		IFA->ExitEditMode();
	}

	FEditorDelegates::EndPIE.RemoveAll(this);
		
	FoliageMeshList.Empty();

	// Call base Exit method to ensure proper cleanup
	FEdMode::Exit();
}

EFoliageEditingState FEdModeFoliage::GetEditingState() const
{
	UWorld* World = GetWorld();

	if (GEditor->bIsSimulatingInEditor)
	{
		return EFoliageEditingState::SIEWorld;
	}
	else if (GEditor->PlayWorld != NULL)
	{
		return EFoliageEditingState::PIEWorld;
	}
	else if (World == nullptr)
	{
		return EFoliageEditingState::Unknown;
	}

	return EFoliageEditingState::Enabled;
}

void FEdModeFoliage::OnEndPIE(const bool bIsSimulating)
{
	if (bIsSimulating)
	{
		PopulateFoliageMeshList();
	}
}

void FEdModeFoliage::PostUndo()
{
	FEdMode::PostUndo();

	PopulateFoliageMeshList();
}

/** When the user changes the active streaming level with the level browser */
void FEdModeFoliage::NotifyNewCurrentLevel()
{
	PopulateFoliageMeshList();
}

void FEdModeFoliage::NotifyLevelAddedToWorld(ULevel* InLevel, UWorld* InWorld)
{
	PopulateFoliageMeshList();
}

void FEdModeFoliage::NotifyLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld)
{
	PopulateFoliageMeshList();
}

void FEdModeFoliage::NotifyAssetRemoved(const FAssetData& AssetInfo)
{
	//TODO: This is not properly removing from the foliage actor. However, when we reload it will skip it.
	//We need to properly fix this, but for now this prevents the crash
	if (UFoliageType* FoliageType = Cast<UFoliageType>(AssetInfo.GetAsset()))
	{
		PopulateFoliageMeshList();
	}
	else if (UBlueprint* Blueprint = Cast<UBlueprint>(AssetInfo.GetAsset()))
	{
		PopulateFoliageMeshList();
	}
}

void FEdModeFoliage::NotifyActorSelectionChanged(bool bSelect, const TArray<AActor*>& Selection)
{
	if (Selection.Num() == 0)
	{
		return;
	}

	GEditor->GetSelectedActors()->Modify();
	for (AActor* Actor : Selection)
	{
        const bool bNotify = false;
		const bool bSelectEvenIfHidden = true;
		GEditor->SelectActor(Actor, bSelect, bNotify, bSelectEvenIfHidden);
	}
	
	// Defer Notification if we are in a selection update scope
	bHasDeferredSelectionNotification = UpdateSelectionCounter > 0;

	if (!bHasDeferredSelectionNotification)
	{
		GEditor->NoteSelectionChange();
	}
}

/** When the user changes the current tool in the UI */
void FEdModeFoliage::HandleToolChanged()
{
	if (UISettings.GetSelectToolSelected() || UISettings.GetLassoSelectToolSelected())
	{
		ApplySelection(GetWorld(), true);
	}
	else
	{
		ApplySelection(GetWorld(), false);
	}

	OnToolChanged.Broadcast();
}

void FEdModeFoliage::ClearAllToolSelection()
{
	UISettings.SetEraseToolSelected(false);
	UISettings.SetLassoSelectToolSelected(false);
	UISettings.SetPaintToolSelected(false);
	UISettings.SetReapplyToolSelected(false);
	UISettings.SetSelectToolSelected(false);
	UISettings.SetPaintBucketToolSelected(false);
	UISettings.SetIsInQuickEraseMode(false);
	UISettings.SetIsInQuickSingleInstantiationMode(false);
	UISettings.SetIsInSingleInstantiationMode(false);
}

void FEdModeFoliage::ForEachFoliageInfo(UWorld* InWorld, const UFoliageType* FoliageType, const FSphere& BrushSphere, TFunctionRef<bool(AInstancedFoliageActor* IFA, FFoliageInfo* FoliageInfo, const UFoliageType* FoliageType)> InOperation)
{
	auto IFAOperation = [&FoliageType, &InOperation](APartitionActor* Actor)
	{
		AInstancedFoliageActor* IFA = Cast<AInstancedFoliageActor>(Actor);
		FFoliageInfo* FoliageInfo = IFA ? IFA->FindInfo(FoliageType) : nullptr;
		if (FoliageInfo)
		{
			return InOperation(IFA, FoliageInfo, FoliageType);
		}
		return true;
	};

	if (UActorPartitionSubsystem* ActorPartitionSubsystem = InWorld->GetSubsystem<UActorPartitionSubsystem>())
	{	
		const FBox BrushSphereBounds(BrushSphere.Center - BrushSphere.W, BrushSphere.Center + BrushSphere.W);
		ActorPartitionSubsystem->ForEachRelevantActor(AInstancedFoliageActor::StaticClass(), BrushSphereBounds, IFAOperation);
	}
}

void FEdModeFoliage::OnSetPaint()
{
	ClearAllToolSelection();
	UISettings.SetPaintToolSelected(true);
	HandleToolChanged();
}

void FEdModeFoliage::OnSetReapplySettings()
{
	ClearAllToolSelection();
	UISettings.SetReapplyToolSelected(true);
	HandleToolChanged();
}

void FEdModeFoliage::OnSetSelectInstance()
{
	ClearAllToolSelection();
	UISettings.SetSelectToolSelected(true);
	HandleToolChanged();
}

void FEdModeFoliage::OnSetLasso()
{
	ClearAllToolSelection();
	UISettings.SetLassoSelectToolSelected(true);
	HandleToolChanged();
}

void FEdModeFoliage::OnSetPaintFill()
{
	ClearAllToolSelection();
	UISettings.SetPaintBucketToolSelected(true);
	HandleToolChanged();
}

void FEdModeFoliage::OnSetErase()
{
	ClearAllToolSelection();
	UISettings.SetPaintToolSelected(true);
	UISettings.SetEraseToolSelected(true);
	HandleToolChanged();
}

void FEdModeFoliage::OnSetPlace()
{
	ClearAllToolSelection();
	UISettings.SetPaintToolSelected(true);
	UISettings.SetIsInSingleInstantiationMode(true);
	HandleToolChanged();
}

void FEdModeFoliage::OnReflectSelectionInPalette()
{
	StaticCastSharedPtr<FFoliageEdModeToolkit>(Toolkit)->ReflectSelectionInPalette();
}

bool FEdModeFoliage::DisallowMouseDeltaTracking() const
{
	// We never want to use the mouse delta tracker while painting
	return bToolActive;
}

void FEdModeFoliage::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	bool bAnyFoliageTypeReplaced = false;

	UWorld* World = GetWorld();
	for (TActorIterator<AInstancedFoliageActor> It(World); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		for (auto& ReplacementPair : ReplacementMap)
		{
			if (UFoliageType* ReplacedFoliageType = Cast<UFoliageType>(ReplacementPair.Key))
			{
				TUniqueObj<FFoliageInfo> FoliageInfo;
				if (IFA->RemoveFoliageInfoAndCopyValue(ReplacedFoliageType, FoliageInfo))
				{
					// Re-add the unique mesh info associated with the replaced foliage type
					UFoliageType* ReplacementFoliageType = Cast<UFoliageType>(ReplacementPair.Value);
					TUniqueObj<FFoliageInfo>& NewFoliageInfo = IFA->AddFoliageInfo(ReplacementFoliageType, MoveTemp(FoliageInfo));
					NewFoliageInfo->ReallocateClusters(ReplacementFoliageType);

					bAnyFoliageTypeReplaced = true;
				}
			}
		}
	}

	if (bAnyFoliageTypeReplaced)
	{
		PopulateFoliageMeshList();
	}
}

void FEdModeFoliage::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	if (!IsEditingEnabled())
	{
		return;
	}

	if (bToolActive)
	{
		ApplyBrush(ViewportClient);
	}

	FEdMode::Tick(ViewportClient, DeltaTime);

	if (UISettings.GetSelectToolSelected() || UISettings.GetLassoSelectToolSelected())
	{
		// Update pivot
		UpdateWidgetLocationToInstanceSelection();
	}

	// Update the position and size of the brush component
	if (bBrushTraceValid && (UISettings.GetPaintToolSelected() || UISettings.GetReapplyToolSelected() || UISettings.GetLassoSelectToolSelected()))
	{
		// Scale adjustment is due to default sphere SM size.
		FTransform BrushTransform = FTransform(FQuat::Identity, BrushLocation, FVector(GetPaintingBrushRadius() * 0.00625f));
		SphereBrushComponent->SetRelativeTransform(BrushTransform);

		static FColor BrushSingleInstanceModeHighlightColor = FColor::Green;
		FColor BrushHighlightColor = UISettings.IsInAnySingleInstantiationMode() ? BrushSingleInstanceModeHighlightColor : BrushDefaultHighlightColor;
		if (BrushCurrentHighlightColor != BrushHighlightColor)
		{
			BrushCurrentHighlightColor = BrushHighlightColor;
			BrushMID->SetVectorParameterValue(FoliageBrushHighlightColorParamName, BrushHighlightColor);
		}
		
		if (!SphereBrushComponent->IsRegistered())
		{
			SphereBrushComponent->RegisterComponentWithWorld(ViewportClient->GetWorld());
		}
	}
	else
	{
		if (SphereBrushComponent->IsRegistered())
		{
			SphereBrushComponent->UnregisterComponent();
		}
	}
}

static TSet<AInstancedFoliageActor*> CurrentFoliageTraceBrushAffectedIFAs;

void FEdModeFoliage::StartFoliageBrushTrace(FEditorViewportClient* ViewportClient, class UViewportInteractor* Interactor)
{
	if (!bToolActive)
	{
		GEditor->BeginTransaction(NSLOCTEXT("UnrealEd", "FoliageMode_EditTransaction", "Foliage Editing"));
		PreApplyBrush();
		ApplyBrush(ViewportClient);

		if (UISettings.IsInAnySingleInstantiationMode())
		{
			EndFoliageBrushTrace();
		}
		else
		{
			bToolActive = true;
		}
	}
}

void FEdModeFoliage::EndFoliageBrushTrace()
{
	GEditor->EndTransaction();
	InstanceSnapshot.Empty();
	LandscapeLayerCaches.Empty();
	bToolActive = false;
	bBrushTraceValid = false;

	for (auto& FoliageMeshUI : FoliageMeshList)
	{
		UFoliageType* Settings = FoliageMeshUI->Settings;

		if (!Settings->IsSelected)
		{
			continue;
		}

		RebuildFoliageTree(Settings);
	}

	CurrentFoliageTraceBrushAffectedIFAs.Empty();
}

/** Trace and update brush position */
void FEdModeFoliage::FoliageBrushTrace(FEditorViewportClient* ViewportClient, const FVector& InRayOrigin, const FVector& InRayDirection)
{
	bBrushTraceValid = false;
	if (ViewportClient == nullptr || ( !ViewportClient->IsMovingCamera() && ViewportClient->IsVisible() ) )
	{
		if (UISettings.GetPaintToolSelected() || UISettings.GetReapplyToolSelected() || UISettings.GetLassoSelectToolSelected())
		{
			const FVector TraceStart(InRayOrigin);
			const FVector TraceEnd(InRayOrigin + InRayDirection * HALF_WORLD_MAX);

			FHitResult Hit;
			UWorld* World = GetWorld();
			static FName NAME_FoliageBrush = FName(TEXT("FoliageBrush"));
			FFoliagePaintingGeometryFilter FilterFunc = FFoliagePaintingGeometryFilter(UISettings);

			if (AInstancedFoliageActor::FoliageTrace(World, Hit, FDesiredFoliageInstance(TraceStart, TraceEnd, /* FoliageType= */ nullptr), NAME_FoliageBrush, /* bReturnFaceIndex */ false, FilterFunc))
			{
				UPrimitiveComponent* PrimComp = Hit.Component.Get();
				if (PrimComp != nullptr && CanPaint(PrimComp->GetComponentLevel()))
				{
					// Adjust the brush location
					BrushLocation = Hit.Location;
					BrushNormal = Hit.Normal;

					// Still want to draw the brush when resizing
					bBrushTraceValid = true;
				}
			}
		}
	}
}

/**
* Called when the mouse is moved over the viewport
*
* @param	InViewportClient	Level editor viewport client that captured the mouse input
* @param	InViewport			Viewport that captured the mouse input
* @param	InMouseX			New mouse cursor X coordinate
* @param	InMouseY			New mouse cursor Y coordinate
*
* @return	true if input was handled
*/
bool FEdModeFoliage::MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 MouseX, int32 MouseY)
{
	// Use mouse capture if there's no other interactor currently tracing brush
	if (IsEditingEnabled())
	{
		// Compute a world space ray from the screen space mouse coordinates
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			ViewportClient->Viewport,
			ViewportClient->GetScene(),
			ViewportClient->EngineShowFlags)
			.SetRealtimeUpdate(ViewportClient->IsRealtime()));

		FSceneView* View = ViewportClient->CalcSceneView(&ViewFamily);
		FViewportCursorLocation MouseViewportRay(View, ViewportClient, MouseX, MouseY);
		BrushTraceDirection = MouseViewportRay.GetDirection();

		FVector BrushTraceStart = MouseViewportRay.GetOrigin();
		if (ViewportClient->IsOrtho())
		{
			BrushTraceStart += -WORLD_MAX * BrushTraceDirection;
		}

		FoliageBrushTrace(ViewportClient, BrushTraceStart, BrushTraceDirection);
	}
	return false;
}

/**
* Called when the mouse is moved while a window input capture is in effect
*
* @param	InViewportClient	Level editor viewport client that captured the mouse input
* @param	InViewport			Viewport that captured the mouse input
* @param	InMouseX			New mouse cursor X coordinate
* @param	InMouseY			New mouse cursor Y coordinate
*
* @return	true if input was handled
*/
bool FEdModeFoliage::CapturedMouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 MouseX, int32 MouseY)
{

	//Compute a world space ray from the screen space mouse coordinates
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		ViewportClient->Viewport,
		ViewportClient->GetScene(),
		ViewportClient->EngineShowFlags)
		.SetRealtimeUpdate(ViewportClient->IsRealtime()));

	FSceneView* View = ViewportClient->CalcSceneView(&ViewFamily);
	FViewportCursorLocation MouseViewportRay(View, ViewportClient, MouseX, MouseY);
	BrushTraceDirection = MouseViewportRay.GetDirection();

	FVector BrushTraceStart = MouseViewportRay.GetOrigin();
	if (ViewportClient->IsOrtho())
	{
		BrushTraceStart += -WORLD_MAX * BrushTraceDirection;
	}

	FoliageBrushTrace(ViewportClient, BrushTraceStart, BrushTraceDirection);

	return false;
}

void FEdModeFoliage::GetRandomVectorInBrush(FVector& OutStart, FVector& OutEnd)
{
	// Find Rx and Ry inside the unit circle
	float Ru = (2.f * FMath::FRand() - 1.f);
	float Rv = (2.f * FMath::FRand() - 1.f) * FMath::Sqrt(1.f - FMath::Square(Ru));

	// find random point in circle through brush location on the same plane to brush location hit surface normal
	FVector U, V;
	BrushNormal.FindBestAxisVectors(U, V);
	FVector Point = Ru * U + Rv * V;

	// find distance to surface of sphere brush from this point
	FVector Rw = FMath::Sqrt(FMath::Max(1.f - (FMath::Square(Ru) + FMath::Square(Rv)), 0.001f)) * BrushNormal;

	OutStart = BrushLocation + UISettings.GetRadius() * (Point + Rw);
	OutEnd = BrushLocation + UISettings.GetRadius() * (Point - Rw);
}

static bool IsWithinSlopeAngle(FVector::FReal NormalZ, float MinAngle, float MaxAngle, float Tolerance = SMALL_NUMBER)
{
	const float MaxNormalAngle = FMath::Cos(FMath::DegreesToRadians(MaxAngle));
	const float MinNormalAngle = FMath::Cos(FMath::DegreesToRadians(MinAngle));
	return !(MaxNormalAngle > (NormalZ + Tolerance) || MinNormalAngle < (NormalZ - Tolerance));
}

/** This does not check for overlaps or density */
static bool CheckLocationForPotentialInstance_ThreadSafe(const UFoliageType* Settings, const FVector& Location, const FVector& Normal)
{
	// Check height range
	FDoubleInterval HeightInterval((double)Settings->Height.Min, (double)Settings->Height.Max);
	if (!HeightInterval.Contains(Location.Z))
	{
		return false;
	}

	// Check slope
	// ImpactNormal sometimes is slightly non-normalized, so compare slope with some little deviation
	return IsWithinSlopeAngle(Normal.Z, Settings->GroundSlopeAngle.Min, Settings->GroundSlopeAngle.Max, SMALL_NUMBER);
}

static bool CheckForOverlappingSphere(AInstancedFoliageActor* IFA, const UFoliageType* Settings, const FSphere& Sphere)
{
	if (IFA)
	{
		FFoliageInfo* Info = IFA->FindInfo(Settings);
		if (Info)
		{
			return Info->CheckForOverlappingSphere(Sphere);
		}
	}

	return false;
}

// Returns whether or not there is are any instances overlapping the sphere specified
static bool CheckForOverlappingSphere(UWorld* InWorld, const UFoliageType* Settings, const FSphere& Sphere)
{
	bool bIsOverlappingSphere = false;
	auto CheckForOverlap = [&bIsOverlappingSphere, &Sphere](AInstancedFoliageActor* IFA, FFoliageInfo* FoliageInfo, const UFoliageType* FoliageType) {
		bIsOverlappingSphere = FoliageInfo->CheckForOverlappingSphere(Sphere);
		return !bIsOverlappingSphere;
	};

	FEdModeFoliage::ForEachFoliageInfo(InWorld, Settings, Sphere, CheckForOverlap);
	return bIsOverlappingSphere;
}

static bool CheckLocationForPotentialInstance(UWorld* InWorld, const UFoliageType* Settings, const bool bSingleInstanceMode, const FVector& Location, const FVector& Normal, TArray<FVector>& PotentialInstanceLocations, FFoliageInstanceHash& PotentialInstanceHash)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CheckLocationForPotentialInstance);

	if (CheckLocationForPotentialInstance_ThreadSafe(Settings, Location, Normal) == false)
	{
		return false;
	}

	const float SettingsRadius = Settings->GetRadius(bSingleInstanceMode);

	// Check if we're too close to any other instances
	if (SettingsRadius > 0.f)
	{
		// Check existing instances. Use the Density radius rather than the minimum radius
		if (CheckForOverlappingSphere(InWorld, Settings, FSphere(Location, SettingsRadius)))
		{
			return false;
		}

		// Check with other potential instances we're about to add.
		TArrayView<const FVector> InstanceLocationsView = PotentialInstanceLocations;
		if (PotentialInstanceHash.IsAnyInstanceInSphere([InstanceLocationsView](int32 Index) -> FVector { return InstanceLocationsView[Index]; }, Location, SettingsRadius))
		{
			return false;
		}
	}

	int32 PotentialIdx = PotentialInstanceLocations.Add(Location);
	PotentialInstanceHash.InsertInstance(Location, PotentialIdx);

	return true;
}

static bool CheckVertexColor(const UFoliageType* Settings, const FColor& VertexColor)
{
	for (uint8 ChannelIdx = 0; ChannelIdx < (uint8)EVertexColorMaskChannel::MAX_None; ++ChannelIdx)
	{
		const FFoliageVertexColorChannelMask& Mask = Settings->VertexColorMaskByChannel[ChannelIdx];

		if (Mask.UseMask)
		{
			uint8 ColorChannel = 0;
			switch ((EVertexColorMaskChannel)ChannelIdx)
			{
			case EVertexColorMaskChannel::Red:
				ColorChannel = VertexColor.R;
				break;
			case EVertexColorMaskChannel::Green:
				ColorChannel = VertexColor.G;
				break;
			case EVertexColorMaskChannel::Blue:
				ColorChannel = VertexColor.B;
				break;
			case EVertexColorMaskChannel::Alpha:
				ColorChannel = VertexColor.A;
				break;
			default:
				// Invalid channel value
				continue;
			}

			if (Mask.InvertMask)
			{
				if (ColorChannel > FMath::RoundToInt(Mask.MaskThreshold * 255.f))
				{
					return false;
				}
			}
			else
			{
				if (ColorChannel < FMath::RoundToInt(Mask.MaskThreshold * 255.f))
				{
					return false;
				}
			}
		}
	}

	return true;
}

bool IsLandscapeLayersArrayValid(const TArray<FName>& LandscapeLayersArray)
{
	bool bValid = false;
	for (FName LayerName : LandscapeLayersArray)
	{
		bValid |= LayerName != NAME_None;
	}

	return bValid;
}

bool GetMaxHitWeight(const FVector& Location, UActorComponent* Component, const TArray<FName>& LandscapeLayersArray, FEdModeFoliage::LandscapeLayerCacheData* LandscapeLayerCaches, float& OutMaxHitWeight)
{
	float MaxHitWeight = 0.f;
	if (ULandscapeHeightfieldCollisionComponent* HitLandscapeCollision = Cast<ULandscapeHeightfieldCollisionComponent>(Component))
	{
		if (ULandscapeComponent* HitLandscape = HitLandscapeCollision->GetRenderComponent())
		{
			for (const FName& LandscapeLayerName : LandscapeLayersArray)
			{
				// Cache store mapping between component and weight data
				TMap<ULandscapeComponent*, TArray<uint8> >* LandscapeLayerCache = &LandscapeLayerCaches->FindOrAdd(LandscapeLayerName);;
				TArray<uint8>* LayerCache = &LandscapeLayerCache->FindOrAdd(HitLandscape);
				// TODO: FName to LayerInfo?
				const float HitWeight = HitLandscape->GetLayerWeightAtLocation(Location, HitLandscape->GetLandscapeInfo()->GetLayerInfoByName(LandscapeLayerName), LayerCache);
				MaxHitWeight = FMath::Max(MaxHitWeight, HitWeight);
			}

			OutMaxHitWeight = MaxHitWeight;
			return true;
		}
	}

	return false;
}

bool IsFilteredByWeight(float Weight, float TestValue, bool bExclusionTest = false)
{
	if (bExclusionTest)
	{
		// Exclusion always tests 
		const float WeightNeeded = FMath::Max(SMALL_NUMBER, TestValue);
		return Weight >= WeightNeeded;
	}
	else
	{
		const float WeightNeeded = FMath::Max(SMALL_NUMBER, FMath::Max(TestValue, FMath::FRand()));
		return Weight < WeightNeeded;
	}
}

bool FEdModeFoliage::IsUsingVertexColorMask(const UFoliageType* Settings)
{
	for (uint8 ChannelIdx = 0; ChannelIdx < (uint8)EVertexColorMaskChannel::MAX_None; ++ChannelIdx)
	{
		const FFoliageVertexColorChannelMask& Mask = Settings->VertexColorMaskByChannel[ChannelIdx];
		if (Mask.UseMask)
		{
			return true;
		}
	}

	return false;
}

bool FEdModeFoliage::VertexMaskCheck(const FHitResult& Hit, const UFoliageType* Settings)
{
	if (Hit.FaceIndex != INDEX_NONE && IsUsingVertexColorMask(Settings))
	{
		if (UStaticMeshComponent* HitStaticMeshComponent = Cast<UStaticMeshComponent>(Hit.Component.Get()))
		{
			FColor VertexColor;
			if (FEdModeFoliage::GetStaticMeshVertexColorForHit(HitStaticMeshComponent, Hit.FaceIndex, Hit.ImpactPoint, VertexColor))
			{
				if (!CheckVertexColor(Settings, VertexColor))
				{
					return false;
				}
			}
		}
	}

	return true;
}

void FEdModeFoliage::SetBrushOpacity(const float InOpacity)
{
	static FName OpacityParamName("OpacityAmount");
	BrushMID->SetScalarParameterValue(OpacityParamName, InOpacity);
}

bool LandscapeLayerCheck(const FHitResult& Hit, const UFoliageType* Settings, FEdModeFoliage::LandscapeLayerCacheData& LandscapeLayersCache, float& OutHitWeight)
{
	OutHitWeight = 1.f;
	if (IsLandscapeLayersArrayValid(Settings->LandscapeLayers) && GetMaxHitWeight(Hit.ImpactPoint, Hit.Component.Get(), Settings->LandscapeLayers, &LandscapeLayersCache, OutHitWeight))
	{
		// Reject instance randomly in proportion to weight
		if (IsFilteredByWeight(OutHitWeight, Settings->MinimumLayerWeight))
		{
			return false;
		}
	}

	float HitWeightExclusion = 1.f;
	if (IsLandscapeLayersArrayValid(Settings->ExclusionLandscapeLayers) && GetMaxHitWeight(Hit.ImpactPoint, Hit.Component.Get(), Settings->ExclusionLandscapeLayers, &LandscapeLayersCache, HitWeightExclusion))
	{
		// Reject instance randomly in proportion to weight
		const bool bExclusionTest = true;
		if (IsFilteredByWeight(HitWeightExclusion, Settings->MinimumExclusionLayerWeight, bExclusionTest))
		{
			return false;
		}
	}

	return true;
}

void FEdModeFoliage::CalculatePotentialInstances_ThreadSafe(UWorld* InWorld, const UFoliageType* Settings, const TArray<FDesiredFoliageInstance>* DesiredInstances, TArray<FPotentialInstance> OutPotentialInstances[NUM_INSTANCE_BUCKETS], const FFoliageUISettings* UISettings, const int32 StartIdx, const int32 LastIdx, const FFoliagePaintingGeometryFilter* OverrideGeometryFilter)
{
	LandscapeLayerCacheData LocalCache;

	// Reserve space in buckets for a potential instances 
	for (int32 BucketIdx = 0; BucketIdx < NUM_INSTANCE_BUCKETS; ++BucketIdx)
	{
		auto& Bucket = OutPotentialInstances[BucketIdx];
		Bucket.Reserve(DesiredInstances->Num());
	}

	for (int32 InstanceIdx = StartIdx; InstanceIdx <= LastIdx; ++InstanceIdx)
	{
		const FDesiredFoliageInstance& DesiredInst = (*DesiredInstances)[InstanceIdx];
		FHitResult Hit;
		static FName NAME_AddFoliageInstances = FName(TEXT("AddFoliageInstances"));

		FFoliageTraceFilterFunc TraceFilterFunc;
		if (DesiredInst.PlacementMode == EFoliagePlacementMode::Manual && UISettings != nullptr)
		{
			// Enable geometry filters when painting foliage manually
			TraceFilterFunc = FFoliagePaintingGeometryFilter(*UISettings);
		}

		if (OverrideGeometryFilter)
		{
			TraceFilterFunc = *OverrideGeometryFilter;
		}

		if (AInstancedFoliageActor::FoliageTrace(InWorld, Hit, DesiredInst, NAME_AddFoliageInstances, /* bReturnFaceIndex */ true, TraceFilterFunc, /*bAverageNormal*/ true))
		{
			float HitWeight = 1.f;
			const bool bValidInstance = CheckLocationForPotentialInstance_ThreadSafe(Settings, Hit.ImpactPoint, Hit.ImpactNormal)
				&& VertexMaskCheck(Hit, Settings)
				&& LandscapeLayerCheck(Hit, Settings, LocalCache, HitWeight);

			if (bValidInstance)
			{
				const int32 BucketIndex = FMath::RoundToInt(HitWeight * (float)(NUM_INSTANCE_BUCKETS - 1));
				OutPotentialInstances[BucketIndex].Add(FPotentialInstance(Hit.ImpactPoint, Hit.ImpactNormal, Hit.Component.Get(), HitWeight, DesiredInst));
			}
		}
	}
}

void FEdModeFoliage::CalculatePotentialInstances(UWorld* InWorld, const UFoliageType* Settings, const TArray<FDesiredFoliageInstance>& DesiredInstances, TArray<FPotentialInstance> OutPotentialInstances[NUM_INSTANCE_BUCKETS], LandscapeLayerCacheData* LandscapeLayerCachesPtr, const FFoliageUISettings* UISettings, const FFoliagePaintingGeometryFilter* OverrideGeometryFilter)
{
	SCOPE_CYCLE_COUNTER(STAT_FoliageCalculatePotentialInstance);
	
	LandscapeLayerCacheData LocalCache;
	LandscapeLayerCachesPtr = LandscapeLayerCachesPtr ? LandscapeLayerCachesPtr : &LocalCache;

	// Quick lookup of potential instance locations, used for overlapping check.
	TArray<FVector> PotentialInstanceLocations;
	FFoliageInstanceHash PotentialInstanceHash(7);	// use 128x128 cell size, things like brush radius are typically small
	PotentialInstanceLocations.Empty(DesiredInstances.Num());

	// Reserve space in buckets for a potential instances 
	for (int32 BucketIdx = 0; BucketIdx < NUM_INSTANCE_BUCKETS; ++BucketIdx)
	{
		auto& Bucket = OutPotentialInstances[BucketIdx];
		Bucket.Reserve(DesiredInstances.Num());
	}

	const bool bSingleIntanceMode = UISettings ? UISettings->IsInAnySingleInstantiationMode() : false;
	for (const FDesiredFoliageInstance& DesiredInst : DesiredInstances)
	{
		FFoliageTraceFilterFunc TraceFilterFunc;
		if (DesiredInst.PlacementMode == EFoliagePlacementMode::Manual && UISettings != nullptr)
		{
			// Enable geometry filters when painting foliage manually
			TraceFilterFunc = FFoliagePaintingGeometryFilter(*UISettings);
		}

		if (OverrideGeometryFilter)
		{
			TraceFilterFunc = *OverrideGeometryFilter;
		}

		FHitResult Hit;
		static FName NAME_AddFoliageInstances = FName(TEXT("AddFoliageInstances"));
		if (AInstancedFoliageActor::FoliageTrace(InWorld, Hit, DesiredInst, NAME_AddFoliageInstances, /* bReturnFaceIndex */ true, TraceFilterFunc, /*bAverageNormal*/ true))
		{
			float HitWeight = 1.f;

			UPrimitiveComponent* InstanceBase = Hit.GetComponent();

			if (InstanceBase == nullptr)
			{
				continue;
			}

			ULevel* TargetLevel = InstanceBase->GetComponentLevel();
			// We can paint into new level only if FoliageType is shared
			if (!CanPaint(Settings, TargetLevel))
			{
				continue;
			}

			const bool bValidInstance = CheckLocationForPotentialInstance(InWorld, Settings, bSingleIntanceMode, Hit.ImpactPoint, Hit.ImpactNormal, PotentialInstanceLocations, PotentialInstanceHash)
				&& VertexMaskCheck(Hit, Settings)
				&& LandscapeLayerCheck(Hit, Settings, LocalCache, HitWeight);
			if (bValidInstance)
			{
				const int32 BucketIndex = FMath::RoundToInt(HitWeight * (float)(NUM_INSTANCE_BUCKETS - 1));
				OutPotentialInstances[BucketIndex].Add(FPotentialInstance(Hit.ImpactPoint, Hit.ImpactNormal, InstanceBase, HitWeight, DesiredInst));
			}
		}
	}
}

void FEdModeFoliage::AddInstances(UWorld* InWorld, const TArray<FDesiredFoliageInstance>& DesiredInstances, const FFoliagePaintingGeometryFilter& OverrideGeometryFilter, bool InRebuildFoliageTree)
{
	TMap<const UFoliageType*, TArray<FDesiredFoliageInstance>> SettingsInstancesMap;
	for (const FDesiredFoliageInstance& DesiredInst : DesiredInstances)
	{
		TArray<FDesiredFoliageInstance>& Instances = SettingsInstancesMap.FindOrAdd(DesiredInst.FoliageType);
		Instances.Add(DesiredInst);
	}

	for (auto It = SettingsInstancesMap.CreateConstIterator(); It; ++It)
	{
		const UFoliageType* FoliageType = It.Key();

		const TArray<FDesiredFoliageInstance>& Instances = It.Value();
		AddInstancesImp(InWorld, FoliageType, Instances, TArray<int32>(), 1.f, nullptr, nullptr, &OverrideGeometryFilter, InRebuildFoliageTree);
	}
}

static void SpawnFoliageInstance(UWorld* InWorld, const UFoliageType* Settings, const FFoliageUISettings* UISettings, const TArray<FFoliageInstance>& PlacedInstances, bool InRebuildFoliageTree)
{
	SCOPE_CYCLE_COUNTER(STAT_FoliageSpawnInstance);

	TMap<AInstancedFoliageActor*, TArray<const FFoliageInstance*>> PerIFAPlacedInstances;
	const bool bSpawnInCurrentLevel = UISettings && UISettings->GetIsInSpawnInCurrentLevelMode();
	ULevel* CurrentLevel = InWorld->GetCurrentLevel();
	const bool bCreate = true;
	for (const FFoliageInstance& PlacedInstance : PlacedInstances)
	{
		ULevel* LevelHint = bSpawnInCurrentLevel ? CurrentLevel : PlacedInstance.BaseComponent ? PlacedInstance.BaseComponent->GetComponentLevel() : nullptr;
		if (AInstancedFoliageActor* IFA = AInstancedFoliageActor::Get(InWorld, bCreate, LevelHint, PlacedInstance.Location))
		{
			PerIFAPlacedInstances.FindOrAdd(IFA).Add(&PlacedInstance);
		}
	}

	for (const auto& PlacedLevelInstances : PerIFAPlacedInstances)
	{
		AInstancedFoliageActor* IFA = PlacedLevelInstances.Key;

		CurrentFoliageTraceBrushAffectedIFAs.Add(IFA);

		FFoliageInfo* Info = nullptr;
		UFoliageType* FoliageSettings = IFA->AddFoliageType(Settings, &Info);

		Info->AddInstances(FoliageSettings, PlacedLevelInstances.Value);
		if (InRebuildFoliageTree)
		{
			Info->Refresh(true, false);
		}
	}
}

void FEdModeFoliage::RebuildFoliageTree(const UFoliageType* Settings)
{
	for (AInstancedFoliageActor* IFA : CurrentFoliageTraceBrushAffectedIFAs)
	{
		if (IFA != nullptr)
		{
			FFoliageInfo* FoliageInfo = IFA->FindInfo(Settings);

			if (FoliageInfo != nullptr)
			{
				FoliageInfo->Refresh(true, false);
			}
		}
	}
}

void FEdModeFoliage::BeginSelectionUpdate()
{
	UpdateSelectionCounter++;
}

void FEdModeFoliage::EndSelectionUpdate()
{
	check(UpdateSelectionCounter > 0);
	UpdateSelectionCounter--;
	if (UpdateSelectionCounter == 0 && bHasDeferredSelectionNotification)
	{
		GEditor->NoteSelectionChange();
		bHasDeferredSelectionNotification = false;
	}
}

bool FEdModeFoliage::AddInstancesImp(UWorld* InWorld, const UFoliageType* Settings, const TArray<FDesiredFoliageInstance>& DesiredInstances, const TArray<int32>& ExistingInstanceBuckets, const float Pressure, LandscapeLayerCacheData* LandscapeLayerCachesPtr, const FFoliageUISettings* UISettings, const FFoliagePaintingGeometryFilter* OverrideGeometryFilter, bool InRebuildFoliageTree)
{
	SCOPE_CYCLE_COUNTER(STAT_FoliageAddInstanceImp);
	
	if (DesiredInstances.Num() == 0)
	{
		return false;
	}

	TArray<FPotentialInstance> PotentialInstanceBuckets[NUM_INSTANCE_BUCKETS];
	if (DesiredInstances[0].PlacementMode == EFoliagePlacementMode::Manual)
	{
		CalculatePotentialInstances(InWorld, Settings, DesiredInstances, PotentialInstanceBuckets, LandscapeLayerCachesPtr, UISettings, OverrideGeometryFilter);
	}
	else
	{
		//@TODO: actual threaded part coming, need parts of this refactor sooner for content team
		CalculatePotentialInstances_ThreadSafe(InWorld, Settings, &DesiredInstances, PotentialInstanceBuckets, nullptr, 0, DesiredInstances.Num() - 1, OverrideGeometryFilter);

		// Existing foliage types in the palette  we want to override any existing mesh settings with the procedural settings.
		TMap<AInstancedFoliageActor*, TArray<const UFoliageType*>> UpdatedTypesByIFA;
		ULevel* CurrentLevel = InWorld->GetCurrentLevel();
		for (TArray<FPotentialInstance>& Bucket : PotentialInstanceBuckets)
		{
			for (auto& PotentialInst : Bucket)
			{
				FFoliageInstance Inst;
				PotentialInst.PlaceInstance(InWorld, Settings, Inst, true);

				ULevel* LevelHint = PotentialInst.HitComponent ? PotentialInst.HitComponent->GetComponentLevel() : CurrentLevel;
				check(LevelHint);
				AInstancedFoliageActor* TargetIFA = AInstancedFoliageActor::Get(InWorld, true, LevelHint, Inst.Location);

				// Update the type in the IFA if needed
				TArray<const UFoliageType*>& UpdatedTypes = UpdatedTypesByIFA.FindOrAdd(TargetIFA);
				if (!UpdatedTypes.Contains(PotentialInst.DesiredInstance.FoliageType))
				{
					UpdatedTypes.Add(PotentialInst.DesiredInstance.FoliageType);
					TargetIFA->AddFoliageType(PotentialInst.DesiredInstance.FoliageType);
				}
			}
		}
	}

	bool bPlacedInstances = false;

	for (int32 BucketIdx = 0; BucketIdx < NUM_INSTANCE_BUCKETS; BucketIdx++)
	{
		TArray<FPotentialInstance>& PotentialInstances = PotentialInstanceBuckets[BucketIdx];
		float BucketFraction = (float)(BucketIdx + 1) / (float)NUM_INSTANCE_BUCKETS;

		// We use the number that actually succeeded in placement (due to parameters) as the target
		// for the number that should be in the brush region.
		const int32 BucketOffset = (ExistingInstanceBuckets.Num() ? ExistingInstanceBuckets[BucketIdx] : 0);
		int32 AdditionalInstances = FMath::Clamp<int32>(FMath::RoundToInt(BucketFraction * (float)(PotentialInstances.Num() - BucketOffset) * Pressure), 0, PotentialInstances.Num());

		{
			SCOPE_CYCLE_COUNTER(STAT_FoliageSpawnInstance);

			TArray<FFoliageInstance> PlacedInstances;
			PlacedInstances.Reserve(AdditionalInstances);

			for (int32 Idx = 0; Idx < AdditionalInstances; Idx++)
			{
				FPotentialInstance& PotentialInstance = PotentialInstances[Idx];
				FFoliageInstance Inst;
				if (PotentialInstance.PlaceInstance(InWorld, Settings, Inst))
				{
					Inst.ProceduralGuid = PotentialInstance.DesiredInstance.ProceduralGuid;
					Inst.BaseComponent = PotentialInstance.HitComponent;
					PlacedInstances.Add(MoveTemp(Inst));
					bPlacedInstances = true;
				}
			}

			SpawnFoliageInstance(InWorld, Settings, UISettings, PlacedInstances, InRebuildFoliageTree);
		}
	}

	return bPlacedInstances;
}

bool FEdModeFoliage::AddSingleInstanceForBrush(UWorld* InWorld, const UFoliageType* Settings, float Pressure)
{
	SCOPE_CYCLE_COUNTER(STAT_FoliageAddInstanceBrush);

	TArray<FDesiredFoliageInstance> DesiredInstances;
	DesiredInstances.Reserve(1);

	// Simply generate a start/end around the brush location so the line check will hit the brush location
	FVector Start = BrushLocation + BrushNormal;
	FVector End = BrushLocation - BrushNormal;

	FDesiredFoliageInstance* DesiredInstance = new (DesiredInstances)FDesiredFoliageInstance(Start, End, Settings);

	// We do not apply the density limitation based on the brush size
	TArray<int32> ExistingInstanceBuckets;
	ExistingInstanceBuckets.AddZeroed(NUM_INSTANCE_BUCKETS);

	return AddInstancesImp(InWorld, Settings, DesiredInstances, ExistingInstanceBuckets, Pressure, &LandscapeLayerCaches, &UISettings, nullptr, false);
}

/** Add instances inside the brush to match DesiredInstanceCount */
void FEdModeFoliage::AddInstancesForBrush(UWorld* InWorld, const UFoliageType* Settings, const FSphere& BrushSphere, int32 DesiredInstanceCount, float Pressure)
{
	SCOPE_CYCLE_COUNTER(STAT_FoliageAddInstanceBrush);

	UWorld* World = GetWorld();
	const bool bHasValidLandscapeLayers = IsLandscapeLayersArrayValid(Settings->LandscapeLayers);

	TArray<int32> ExistingInstanceBuckets;
	ExistingInstanceBuckets.AddZeroed(NUM_INSTANCE_BUCKETS);
	int32 NumExistingInstances = 0;
	
	auto AddingInstances = [this, &BrushSphere, &NumExistingInstances, &bHasValidLandscapeLayers, &ExistingInstanceBuckets](AInstancedFoliageActor* IFA, FFoliageInfo* FoliageInfo, const UFoliageType* FoliageType) {
	
		TArray<int32> ExistingInstances;
		FoliageInfo->GetInstancesInsideSphere(BrushSphere, ExistingInstances);
		NumExistingInstances += ExistingInstances.Num();

		if (bHasValidLandscapeLayers)
		{
			// Find the landscape weights of existing ExistingInstances
			for (int32 Idx : ExistingInstances)
			{
				FFoliageInstance& Instance = FoliageInfo->Instances[Idx];
				auto InstanceBasePtr = IFA->InstanceBaseCache.GetInstanceBasePtr(Instance.BaseId);
				float HitWeight;
				if (GetMaxHitWeight(Instance.Location, InstanceBasePtr.Get(), FoliageType->LandscapeLayers, &LandscapeLayerCaches, HitWeight))
				{
					// Add count to bucket.
					ExistingInstanceBuckets[FMath::RoundToInt(HitWeight * (float)(NUM_INSTANCE_BUCKETS - 1))]++;
				}
			}
		}
		else
		{
			// When not tied to a layer, put all the ExistingInstances in the last bucket.
			ExistingInstanceBuckets[NUM_INSTANCE_BUCKETS - 1] = NumExistingInstances;
		}

		return true;
	};
	
	ForEachFoliageInfo(InWorld, Settings, BrushSphere, AddingInstances);

	if (DesiredInstanceCount > NumExistingInstances)
	{
		TArray<FDesiredFoliageInstance> DesiredInstances;	//we compute instances for the brush
		DesiredInstances.Reserve(DesiredInstanceCount);

		for (int32 DesiredIdx = 0; DesiredIdx < DesiredInstanceCount; DesiredIdx++)
		{
			FVector Start, End;
			GetRandomVectorInBrush(Start, End);
			FDesiredFoliageInstance* DesiredInstance = new (DesiredInstances)FDesiredFoliageInstance(Start, End, Settings);
		}

		AddInstancesImp(InWorld, Settings, DesiredInstances, ExistingInstanceBuckets, Pressure, &LandscapeLayerCaches, &UISettings, nullptr, false);
	}
}

/** Remove instances inside the brush to match DesiredInstanceCount */
void FEdModeFoliage::RemoveInstancesForBrush(UWorld* InWorld, const UFoliageType* Settings, const FSphere& BrushSphere, int32 DesiredInstanceCount, float Pressure)
{
	SCOPE_CYCLE_COUNTER(STAT_FoliageRemoveInstanceBrush);

	struct FInstancesToRemove
	{
		FFoliageInfo* FoliageInfo;
		TArray<int32> InstancesToRemove;
	};
	TArray<FInstancesToRemove> PotentialInstancesToRemove;
	int32 PotentialInstancesToRemoveCount = 0;

	// Get Brush intersecting instances per FoliageInfo (per IFA)
	ForEachFoliageInfo(InWorld, Settings, BrushSphere, [&BrushSphere, &PotentialInstancesToRemove, &PotentialInstancesToRemoveCount](AInstancedFoliageActor* IFA, FFoliageInfo* FoliageInfo, const UFoliageType* FoliageType)
	{
		TArray<int32> InstancesInsideSphere;
		FoliageInfo->GetInstancesInsideSphere(BrushSphere, InstancesInsideSphere);
		if (InstancesInsideSphere.Num() > 0)
		{
			PotentialInstancesToRemoveCount += InstancesInsideSphere.Num();
			PotentialInstancesToRemove.Add({ FoliageInfo, MoveTemp(InstancesInsideSphere) });
		}
		return true;
	});

	// Calculate number of Instances to remove based on desired instance count
	const int32 InstancesToRemoveCount = FMath::RoundToInt((float)(PotentialInstancesToRemoveCount - DesiredInstanceCount) * Pressure);
	if (InstancesToRemoveCount <= 0)
	{
		return;
	}

	// Calculate InstancesToKeep
	const int32 InstancesToKeepCount = PotentialInstancesToRemoveCount - InstancesToRemoveCount;
	
	// Remove InstancesToKeep from the PotentialInstancesToRemove randomly so that they don't get removed
	for (int32 i = 0; i < InstancesToKeepCount; i++)
	{
		const int32 RemoveIndex = FMath::Rand() % PotentialInstancesToRemoveCount;
		int32 StartIndex = 0;
		
		// Iterate through IFAs to find the RemoveIndex
		for (auto& [FoliageInfo, InstancesToRemove] : PotentialInstancesToRemove)
		{
			const int32 LocalRemoveIndex = RemoveIndex - StartIndex;
			if (InstancesToRemove.IsValidIndex(LocalRemoveIndex))
			{
				InstancesToRemove.RemoveAtSwap(LocalRemoveIndex, EAllowShrinking::No);
				break;
			}
			StartIndex += InstancesToRemove.Num();
		}
		PotentialInstancesToRemoveCount--;
	}
	
	FFoliagePaintingGeometryFilter GeometryFilterFunc(UISettings);

	// Filter PotentialInstancesToRemove
	for (auto& [FoliageInfo, InstancesToRemove] : PotentialInstancesToRemove)
	{
		AInstancedFoliageActor* IFA = FoliageInfo->IFA;

		for (int32 Idx = 0; Idx < InstancesToRemove.Num(); Idx++)
		{
			auto BaseId = FoliageInfo->Instances[InstancesToRemove[Idx]].BaseId;
			auto BasePtr = IFA->InstanceBaseCache.GetInstanceBasePtr(BaseId);
			UPrimitiveComponent* Base = Cast<UPrimitiveComponent>(BasePtr.Get());

			// Check if instance is candidate for removal based on filter settings
			if (Base && !GeometryFilterFunc(Base))
			{
				// Instance should not be removed, so remove it from the removal list.
				InstancesToRemove.RemoveAtSwap(Idx, EAllowShrinking::No);
				Idx--;
			}
		}

		// Remove InstancesToRemove to reduce it to desired count
		if (InstancesToRemove.Num() > 0)
		{
			CurrentFoliageTraceBrushAffectedIFAs.Add(FoliageInfo->IFA);

			FoliageInfo->RemoveInstances(InstancesToRemove, false);
		}
	}
}

void FEdModeFoliage::SelectInstanceAtLocation(UWorld* InWorld, const UFoliageType* Settings, const FVector& Location, bool bSelect)
{
	FEdModeFoliageSelectionUpdate Scope(this);
	const float WidthOfSphere = 10.f;
	FSphere SphereBounds(Location,WidthOfSphere);

	auto SelectInstances = [&Location, &bSelect](AInstancedFoliageActor* IFA, FFoliageInfo* FoliageInfo, const UFoliageType* FoliageType) {

		int32 Instance;
		bool bResult;
		FoliageInfo->GetInstanceAtLocation(Location, Instance, bResult);
		if (bResult)
		{
			TArray<int32> Instances;
			Instances.Add(Instance);
			FoliageInfo->SelectInstances(bSelect, Instances);
		}

		return true;
	};
	ForEachFoliageInfo(InWorld, Settings, SphereBounds, SelectInstances);
	
}

void FEdModeFoliage::SelectInstancesForBrush(UWorld* InWorld, const UFoliageType* Settings, const FSphere& BrushSphere, bool bSelect)
{
	FEdModeFoliageSelectionUpdate Scope(this);
	
	auto SelectInstances = [&BrushSphere, &bSelect](AInstancedFoliageActor* IFA, FFoliageInfo* FoliageInfo, const UFoliageType* FoliageType) {

		TArray<int32> Instances;
		FoliageInfo->GetInstancesInsideSphere(BrushSphere, Instances);
		if (Instances.Num() == 0)
		{
			return true;
		}
		FoliageInfo->SelectInstances(bSelect, Instances);

		return true;
	};

	ForEachFoliageInfo(InWorld, Settings, BrushSphere, SelectInstances);
}

void RefreshSceneOutliner()
{
	// SceneOutliner Refresh
	TWeakPtr<class ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();
	if (LevelEditor.IsValid())
	{
		TArray<TWeakPtr<class ISceneOutliner>> SceneOutlinerPtrs = LevelEditor.Pin()->GetAllSceneOutliners();

		for (TWeakPtr<class ISceneOutliner> SceneOutlinerPtr : SceneOutlinerPtrs)
		{
			if (TSharedPtr<class ISceneOutliner> SceneOutlinerPin = SceneOutlinerPtr.Pin())
			{
				SceneOutlinerPin->FullRefresh();
			}
		}
	}
}

void FEdModeFoliage::ExcludeFoliageActors(const TArray<const UFoliageType *>& FoliageTypes, bool bOnlyCurrentLevel)
{
	FScopedTransaction Transaction(LOCTEXT("ExcludeFoliageActors", "Exclude Actors from Foliage"));
	TMap<TSubclassOf<AActor>, const UFoliageType_Actor*> ActorFoliageTypes;
	for (const UFoliageType* FoliageType : FoliageTypes)
	{
		if (const UFoliageType_Actor* ActorFoliageType = Cast<UFoliageType_Actor>(FoliageType))
		{
			ActorFoliageTypes.Add(ActorFoliageType->ActorClass, ActorFoliageType);
		}
	}

	// Go through all sub-levels
	UWorld* World = GetWorld();
	for (TActorIterator<AInstancedFoliageActor> It(World); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		//@todo_ow: Current level check doesn't apply in WP
		if (!bOnlyCurrentLevel || IFA->GetLevel() == World->GetCurrentLevel())
		{
			for (auto& Pair : ActorFoliageTypes)
			{
				if (FFoliageInfo* FoliageInfoPtr = IFA->FindInfo(Pair.Value))
				{
					IFA->Modify();
					FoliageInfoPtr->ExcludeActors();
					OnInstanceCountUpdated(Pair.Value);
				}
			}
		}
	}

	RefreshSceneOutliner();
}

void FEdModeFoliage::IncludeNonFoliageActors(const TArray<const UFoliageType*>& FoliageTypes, bool bOnlyCurrentLevel)
{
	FScopedTransaction Transaction(LOCTEXT("IncludeNonFoliageActors", "Include Actors into Foliage"));
	TMap<const UClass*, const UFoliageType_Actor*> ActorFoliageTypes;
	for (const UFoliageType* FoliageType : FoliageTypes)
	{
		if (const UFoliageType_Actor* ActorFoliageType = Cast<UFoliageType_Actor>(FoliageType))
		{
			ActorFoliageTypes.Add(ActorFoliageType->ActorClass.Get(), ActorFoliageType);
		}
	}

	TSet<const UFoliageType*> UpdateFoliageTypes;

	// Go through all sub-levels
	UWorld* World = GetWorld();
	const int32 NumLevels = World->GetNumLevels();
	for (int32 LevelIdx = 0; LevelIdx < NumLevels; ++LevelIdx)
	{
		ULevel* Level = World->GetLevel(LevelIdx);
		if (!bOnlyCurrentLevel || Level == World->GetCurrentLevel())
		{
			AInstancedFoliageActor* IFA = nullptr;

			for (auto ActorIterator = Level->Actors.CreateIterator(); ActorIterator; ++ActorIterator)
			{
				AActor* CurrentActor = *ActorIterator;
				if (CurrentActor)
				{
					if (const UFoliageType_Actor** FoliageTypePtr = ActorFoliageTypes.Find(CurrentActor->GetClass()))
					{
						if (const UFoliageType* FoliageType = *FoliageTypePtr)
						{
							if (IFA == nullptr)
							{
								IFA = AInstancedFoliageActor::GetInstancedFoliageActorForLevel(Level, true);
							}

							

							FFoliageInfo* FoliageInfo;
							IFA->Modify();
							IFA->AddFoliageType(FoliageType, &FoliageInfo);
							if (FoliageInfo)
							{

								FoliageInfo->IncludeActor(FoliageType, CurrentActor);
								UpdateFoliageTypes.Add(FoliageType);
							}
						}
					}
				}
			}
		}
	}

	for (const UFoliageType* FoliageType : UpdateFoliageTypes)
	{
		OnInstanceCountUpdated(FoliageType);
	}

	RefreshSceneOutliner();
}

void FEdModeFoliage::SelectInstances(const TArray<const UFoliageType *>& FoliageTypes, bool bSelect)
{
	FEdModeFoliageSelectionUpdate Scope(this);
	for (const UFoliageType* FoliageType : FoliageTypes)
	{
		SelectInstances(FoliageType, bSelect);
	}
}

void FEdModeFoliage::SelectInstances(const UFoliageType* Settings, bool bSelect)
{
	SelectInstances(GetWorld(), Settings, bSelect);
}

void FEdModeFoliage::FocusSelectedInstances() const
{
	UWorld* World = GetWorld();
	FBox SelectionBoundingBox(EForceInit::ForceInit);
	for (TActorIterator<AInstancedFoliageActor> It(World); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		SelectionBoundingBox += IFA->GetSelectionBoundingBox();
	}
	GEditor->MoveViewportCamerasToBox(SelectionBoundingBox, true);
}

void FEdModeFoliage::SelectInstances(UWorld* InWorld, bool bSelect)
{
	FEdModeFoliageSelectionUpdate Scope(this);
	for (auto& FoliageMeshUI : FoliageMeshList)
	{
		UFoliageType* Settings = FoliageMeshUI->Settings;

		if (bSelect && !Settings->IsSelected)
		{
			continue;
		}

		SelectInstances(InWorld, Settings, bSelect);
	}
}

void FEdModeFoliage::SelectInstances(UWorld* InWorld, const UFoliageType* Settings, bool bSelect)
{
	FEdModeFoliageSelectionUpdate Scope(this);
	for (FFoliageInfoIterator It(InWorld, Settings); It; ++It)
	{
		FFoliageInfo* FoliageInfo = (*It);
		AInstancedFoliageActor* IFA = It.GetActor();

		FoliageInfo->SelectInstances(bSelect);
	}
}

void FEdModeFoliage::ApplySelection(UWorld* InWorld, bool bApply)
{
	GEditor->SelectNone(true, true);

	FEdModeFoliageSelectionUpdate Scope(this);
	for (TActorIterator<AInstancedFoliageActor> It(GetWorld()); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		IFA->ApplySelection(bApply);
	}
}

void FEdModeFoliage::UpdateInstancePartitioning(UWorld* InWorld)
{
	check(!bMoving);
	AInstancedFoliageActor::UpdateInstancePartitioning(InWorld);
}

void FEdModeFoliage::PostTransformSelectedInstances(UWorld* InWorld)
{
	if (!bMoving)
	{
		return;
	}

	bMoving = false;
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		bool bFoundSelection = false;

		IFA->ForEachFoliageInfo([](UFoliageType* FoliageType, FFoliageInfo& FoliageInfo)
		{
			TArray<int32> SelectedIndices = FoliageInfo.SelectedIndices.Array();

			if (SelectedIndices.Num() > 0)
			{
				const bool bFinished = true;
				FoliageInfo.PostMoveInstances(SelectedIndices, bFinished);
			}
			return true; // continue iteration
		});
	}

	UpdateInstancePartitioning(GetWorld());
}

void FEdModeFoliage::TransformSelectedInstances(UWorld* InWorld, const FVector& InDrag, const FRotator& InRot, const FVector& InScale, bool bDuplicate)
{
	bool bDidMoveFoliage = false;
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		bool bFoundSelection = false;

		IFA->ForEachFoliageInfo([this, IFA, &bDidMoveFoliage, &bFoundSelection, InDrag, InRot, InScale, bDuplicate](UFoliageType* FoliageType, FFoliageInfo& FoliageInfo)
		{
			TArray<int32> SelectedIndices = FoliageInfo.SelectedIndices.Array();

			if (SelectedIndices.Num() > 0)
			{
				// Mark actor once we found selection
				if (!bFoundSelection)
				{
					IFA->Modify();
					bFoundSelection = true;
				}

				if (bDuplicate)
				{
					FoliageInfo.DuplicateInstances(FoliageType, SelectedIndices);
					OnInstanceCountUpdated(FoliageType);
				}

				if (!bMoving)
				{
					FoliageInfo.PreMoveInstances(SelectedIndices);
				}
				bDidMoveFoliage = true;

				for (int32 SelectedInstanceIdx : SelectedIndices)
				{
					FFoliageInstance& Instance = FoliageInfo.Instances[SelectedInstanceIdx];
					Instance.Location += InDrag;
					Instance.ZOffset = 0.f;
					Instance.Rotation += InRot;
					Instance.DrawScale3D += (FVector3f)InScale;
				}

				FoliageInfo.PostMoveInstances(SelectedIndices, /*bFinished*/false);
				FoliageInfo.PreMoveInstances(SelectedIndices);
			}
			return true; // continue iteration
		});

		if (bFoundSelection)
		{
			IFA->MarkComponentsRenderStateDirty();
		}
	}

	if (bDidMoveFoliage)
	{
		bMoving = true;
	}
}

bool FEdModeFoliage::GetSelectionLocation(UWorld* InWorld, FVector& OutLocation) const
{
	FBox OutBox(EForceInit::ForceInit);
	bool bHasSelection = false;
	// Go through all sub-levels
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		bHasSelection |= (IFA && IFA->GetSelectionLocation(OutBox));
	}

	if (bHasSelection)
	{
		OutLocation = OutBox.GetCenter();
	}

	return bHasSelection;
}

void FEdModeFoliage::UpdateWidgetLocationToInstanceSelection()
{
	FVector SelectionLocation = FVector::ZeroVector;
	if (GetSelectionLocation(GetWorld(), SelectionLocation))
	{
		Owner->PivotLocation = Owner->SnappedLocation = SelectionLocation;
	}
}

void FEdModeFoliage::RemoveSelectedInstances(UWorld* InWorld)
{
	FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "FoliageMode_EditTransaction", "Foliage Editing"));

	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		bool bHasSelection = false;
		for (auto& MeshPair : IFA->GetFoliageInfos())
		{
			if (MeshPair.Value->SelectedIndices.Num())
			{
				bHasSelection = true;
				break;
			}
		}

		if (bHasSelection)
		{
			IFA->Modify();
			IFA->ForEachFoliageInfo([this](UFoliageType* FoliageType, FFoliageInfo& FoliageInfo)
			{
				if (FoliageInfo.SelectedIndices.Num() > 0)
				{
					TArray<int32> InstancesToDelete = FoliageInfo.SelectedIndices.Array();

					// Make sure that any moving instances to be removed are added back to the hash.
					if (bMoving)
					{
						TArray<int32> MovingInstancesToDelete = FoliageInfo.SelectedIndices.Intersect(FoliageInfo.MovingInstances).Array();

						if (MovingInstancesToDelete.Num())
						{
							FoliageInfo.PostMoveInstances(MovingInstancesToDelete, /*bFinished*/true);
						}
					}

					FoliageInfo.RemoveInstances(InstancesToDelete, true);

					OnInstanceCountUpdated(FoliageType);
				}
				return true; // continue iteration
			});
		}
	}
}

void FEdModeFoliage::GetFoliageTypeFilters(TArray<const UClass*>& OutFilters) const
{
	OutFilters.Add(UFoliageType_InstancedStaticMesh::StaticClass());
	if (GetWorld()->IsPartitionedWorld())
	{
		return;
	}
	// FoliageType Actor only supported in non WorldPartition worlds
	OutFilters.Add(UFoliageType_Actor::StaticClass());
}

void FEdModeFoliage::GetSelectedInstanceFoliageTypes(TArray<const UFoliageType*>& OutFoliageTypes) const
{
	for (TActorIterator<AInstancedFoliageActor> It(GetWorld()); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		bool bHasSelection = false;
		for (auto& MeshPair : IFA->GetFoliageInfos())
		{
			if (MeshPair.Value->SelectedIndices.Num())
			{
				OutFoliageTypes.AddUnique(MeshPair.Key);
			}
		}
	}
}

TAutoConsoleVariable<float> CVarOffGroundTreshold(
	TEXT("foliage.OffGroundThreshold"),
	5.0f,
	TEXT("Maximum distance from base component (in local space) at which instance is still considered as valid"));

void FEdModeFoliage::SelectInvalidInstances(const TArray<const UFoliageType *>& FoliageTypes)
{
	FEdModeFoliageSelectionUpdate Scope(this);
	for (const UFoliageType* FoliageType : FoliageTypes)
	{
		SelectInvalidInstances(FoliageType);
	}
}

void FEdModeFoliage::SelectInvalidInstances(const UFoliageType* Settings)
{
	FEdModeFoliageSelectionUpdate Scope(this);
	UWorld* InWorld = GetWorld();

	FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(FoliageGroundCheck), true);
	QueryParams.bReturnFaceIndex = false;
	FCollisionShape SphereShape;
	SphereShape.SetSphere(0.f);
	float InstanceOffGroundLocalThreshold = CVarOffGroundTreshold.GetValueOnGameThread();

	for (FFoliageInfoIterator It(InWorld, Settings); It; ++It)
	{
		FFoliageInfo* FoliageInfo = (*It);
		AInstancedFoliageActor* IFA = It.GetActor();
		int32 NumInstances = FoliageInfo->Instances.Num();
		TArray<FHitResult> Hits; Hits.Reserve(16);

		TArray<int32> InvalidInstances;
		
		for (int32 InstanceIdx = 0; InstanceIdx < NumInstances; ++InstanceIdx)
		{
			FFoliageInstance& Instance = FoliageInfo->Instances[InstanceIdx];
			UActorComponent* CurrentInstanceBase = IFA->InstanceBaseCache.GetInstanceBasePtr(Instance.BaseId).Get();
			
			bool bInvalidInstance = FoliageInfo->ShouldAttachToBaseComponent() || (!FoliageInfo->ShouldAttachToBaseComponent() && CurrentInstanceBase != nullptr);
			if (FoliageInfo->ShouldAttachToBaseComponent() && CurrentInstanceBase != nullptr)
			{
				FVector InstanceTraceRange = Instance.GetInstanceWorldTransform().TransformVector(FVector(0.f, 0.f, 1000.f));
				FVector Start = Instance.Location + InstanceTraceRange;
				FVector End = Instance.Location - InstanceTraceRange;

				InWorld->SweepMultiByObjectType(Hits, Start, End, FQuat::Identity, FCollisionObjectQueryParams(ECC_WorldStatic), SphereShape, QueryParams);

				for (const FHitResult& Hit : Hits)
				{
					UPrimitiveComponent* HitComponent = Hit.GetComponent();
					check(HitComponent);

					if (HitComponent->IsCreatedByConstructionScript())
					{
						continue;
					}

					UModelComponent* ModelComponent = Cast<UModelComponent>(HitComponent);
					if (ModelComponent)
					{
						ABrush* BrushActor = ModelComponent->GetModel()->FindBrush((FVector3f)Hit.Location);
						if (BrushActor)
						{
							HitComponent = BrushActor->GetBrushComponent();
						}
					}

					if (HitComponent == CurrentInstanceBase)
					{
						FVector InstanceWorldZOffset = Instance.GetInstanceWorldTransform().TransformVector(FVector(0.f, 0.f, Instance.ZOffset));
						FVector::FReal DistanceToGround = FVector::Dist(Instance.Location, Hit.Location + InstanceWorldZOffset);
						FVector::FReal InstanceWorldTreshold = Instance.GetInstanceWorldTransform().TransformVector(FVector(0.f, 0.f, InstanceOffGroundLocalThreshold)).Size();

						if ((DistanceToGround - InstanceWorldTreshold) <= KINDA_SMALL_NUMBER)
						{
							bInvalidInstance = false;
							break;
						}
					}
				}
			}

			if (bInvalidInstance)
			{
				InvalidInstances.Add(InstanceIdx);
			}
		}

		if (InvalidInstances.Num() > 0)
		{
			FoliageInfo->SelectInstances(true, InvalidInstances);
		}
	}
}

void FEdModeFoliage::AdjustBrushRadius(float Multiplier)
{
	if (UISettings.IsInAnySingleInstantiationMode())
	{
		return;
	}

	const float PercentageChange = 0.05f;
	const float CurrentBrushRadius = UISettings.GetRadius();

	float NewValue = CurrentBrushRadius * (1 + PercentageChange * Multiplier);
	UISettings.SetRadius(FMath::Clamp(NewValue, 0.1f, 8192.0f));
}

void FEdModeFoliage::AdjustPaintDensity(float Multiplier)
{
	if (UISettings.IsInAnySingleInstantiationMode())
	{
		return;
	}

	const float AdjustmentAmount = 0.02f;
	const float CurrentDensity = UISettings.GetPaintDensity();

	UISettings.SetPaintDensity(FMath::Clamp(CurrentDensity + AdjustmentAmount * Multiplier, 0.0f, 1.0f));
}

void FEdModeFoliage::AdjustUnpaintDensity(float Multiplier)
{
	if (UISettings.IsInAnySingleInstantiationMode())
	{
		return;
	}

	const float AdjustmentAmount = 0.02f;
	const float CurrentDensity = UISettings.GetUnpaintDensity();

	UISettings.SetUnpaintDensity(FMath::Clamp(CurrentDensity + AdjustmentAmount * Multiplier, 0.0f, 1.0f));
}

void FEdModeFoliage::ReapplyInstancesForBrush(UWorld* InWorld, const UFoliageType* Settings, const FSphere& BrushSphere, float Pressure, bool bSingleInstanceMode)
{
	// Adjust instance density first
	ReapplyInstancesDensityForBrush(InWorld, Settings, BrushSphere, Pressure);

	auto ReapplyInstances = [this, &InWorld, &BrushSphere, &Pressure, &bSingleInstanceMode](AInstancedFoliageActor* IFA, FFoliageInfo* FoliageInfo, const UFoliageType* FoliageType) {
		
		ReapplyInstancesForBrush(InWorld, IFA, FoliageType, FoliageInfo, BrushSphere, Pressure, bSingleInstanceMode);
		return true;
	};
	ForEachFoliageInfo(InWorld, Settings, BrushSphere, ReapplyInstances);
}

/** Reapply instance settings to exiting instances */
void FEdModeFoliage::ReapplyInstancesForBrush(UWorld* InWorld, AInstancedFoliageActor* IFA, const UFoliageType* Settings, FFoliageInfo* FoliageInfo, const FSphere& BrushSphere, float Pressure, bool bSingleInstanceMode)
{
	TArray<int32> ExistingInstances;
	FoliageInfo->GetInstancesInsideSphere(BrushSphere, ExistingInstances);

	const FDoubleInterval HeightInterval(Settings->Height.Min, Settings->Height.Max);
	bool bUpdated = false;
	TArray<int32> UpdatedInstances;
	TSet<int32> InstancesToDelete;

	IFA->Modify();

	for (int32 Idx = 0; Idx < ExistingInstances.Num(); Idx++)
	{
		int32 InstanceIndex = ExistingInstances[Idx];
		FFoliageInstance& Instance = FoliageInfo->Instances[InstanceIndex];

		if ((Instance.Flags & FOLIAGE_Readjusted) == 0)
		{
			// record that we've made changes to this instance already, so we don't touch it again.
			Instance.Flags |= FOLIAGE_Readjusted;

			// See if we need to update the location in the instance hash
			bool bReapplyLocation = false;
			FVector OldInstanceLocation = Instance.Location;

			// remove any Z offset first, so the offset is reapplied to any new 
			if (FMath::Abs(Instance.ZOffset) > KINDA_SMALL_NUMBER)
			{
				Instance.Location = Instance.GetInstanceWorldTransform().TransformPosition(FVector(0, 0, -Instance.ZOffset));
				bReapplyLocation = true;
			}


			// Defer normal reapplication 
			bool bReapplyNormal = false;

			// Reapply normal alignment
			if (Settings->ReapplyAlignToNormal)
			{
				if (Settings->AlignToNormal)
				{
					if ((Instance.Flags & FOLIAGE_AlignToNormal) == 0)
					{
						bReapplyNormal = true;
						bUpdated = true;
					}
				}
				else
				{
					if (Instance.Flags & FOLIAGE_AlignToNormal)
					{
						Instance.Rotation = Instance.PreAlignRotation;
						Instance.Flags &= (~FOLIAGE_AlignToNormal);
						bUpdated = true;
					}
				}
			}

			// Reapply random yaw
			if (Settings->ReapplyRandomYaw)
			{
				if (Settings->RandomYaw)
				{
					if (Instance.Flags & FOLIAGE_NoRandomYaw)
					{
						// See if we need to remove any normal alignment first
						if (!bReapplyNormal && (Instance.Flags & FOLIAGE_AlignToNormal))
						{
							Instance.Rotation = Instance.PreAlignRotation;
							bReapplyNormal = true;
						}
						Instance.Rotation.Yaw = FMath::FRand() * 360.f;
						Instance.Flags &= (~FOLIAGE_NoRandomYaw);
						bUpdated = true;
					}
				}
				else
				{
					if ((Instance.Flags & FOLIAGE_NoRandomYaw) == 0)
					{
						// See if we need to remove any normal alignment first
						if (!bReapplyNormal && (Instance.Flags & FOLIAGE_AlignToNormal))
						{
							Instance.Rotation = Instance.PreAlignRotation;
							bReapplyNormal = true;
						}
						Instance.Rotation.Yaw = 0.f;
						Instance.Flags |= FOLIAGE_NoRandomYaw;
						bUpdated = true;
					}
				}
			}

			// Reapply random pitch angle
			if (Settings->ReapplyRandomPitchAngle)
			{
				// See if we need to remove any normal alignment first
				if (!bReapplyNormal && (Instance.Flags & FOLIAGE_AlignToNormal))
				{
					Instance.Rotation = Instance.PreAlignRotation;
					bReapplyNormal = true;
				}

				Instance.Rotation.Pitch = FMath::FRand() * Settings->RandomPitchAngle;
				Instance.Flags |= FOLIAGE_NoRandomYaw;

				bUpdated = true;
			}

			// Reapply scale
			if (Settings->ReapplyScaling)
			{
				FVector3f NewScale = Settings->GetRandomScale();

				if (Settings->ReapplyScaleX)
				{
					if (Settings->Scaling == EFoliageScaling::Uniform)
					{
						Instance.DrawScale3D = NewScale;
					}
					else
					{
						Instance.DrawScale3D.X = NewScale.X;
					}
					bUpdated = true;
				}

				if (Settings->ReapplyScaleY)
				{
					Instance.DrawScale3D.Y = NewScale.Y;
					bUpdated = true;
				}

				if (Settings->ReapplyScaleZ)
				{
					Instance.DrawScale3D.Z = NewScale.Z;
					bUpdated = true;
				}
			}

			// Reapply ZOffset
			if (Settings->ReapplyZOffset)
			{
				Instance.ZOffset = Settings->ZOffset.Interpolate(FMath::FRand());
				bUpdated = true;
			}

			// Find a ground normal for either normal or ground slope check.
			if (bReapplyNormal || Settings->ReapplyGroundSlope || Settings->ReapplyVertexColorMask || (Settings->ReapplyLandscapeLayers && IsLandscapeLayersArrayValid(Settings->LandscapeLayers)))
			{
				FHitResult Hit;
				static const FName NAME_ReapplyInstancesForBrush = TEXT("ReapplyInstancesForBrush");

				// trace along the mesh's Z axis.
				FVector ZAxis = Instance.Rotation.Quaternion().GetAxisZ();
				FVector Start = Instance.Location + 16.f * ZAxis;
				FVector End = Instance.Location - 16.f * ZAxis;
				if (AInstancedFoliageActor::FoliageTrace(InWorld, Hit, FDesiredFoliageInstance(Start, End, Settings), NAME_ReapplyInstancesForBrush, /* bReturnFaceIndex */ true, FFoliageTraceFilterFunc(), /*bAverageNormal*/ true))
				{
					// Reapply the normal
					if (bReapplyNormal)
					{
						Instance.PreAlignRotation = Instance.Rotation;
						Instance.AlignToNormal(Hit.Normal, Settings->AlignMaxAngle);
					}

					// Cull instances that don't meet the ground slope check.
					if (Settings->ReapplyGroundSlope)
					{
						if (!IsWithinSlopeAngle(Hit.Normal.Z, Settings->GroundSlopeAngle.Min, Settings->GroundSlopeAngle.Max))
						{
							InstancesToDelete.Add(InstanceIndex);
							if (bReapplyLocation)
							{
								// restore the location so the hash removal will succeed
								Instance.Location = OldInstanceLocation;
							}
							continue;
						}
					}

					// Cull instances for the landscape layer
					if (Settings->ReapplyLandscapeLayers && IsLandscapeLayersArrayValid(Settings->LandscapeLayers))
					{
						float HitWeight = 1.f;
						if (GetMaxHitWeight(Hit.Location, Hit.GetComponent(), Settings->LandscapeLayers, &LandscapeLayerCaches, HitWeight))
						{
							if (IsFilteredByWeight(HitWeight, Settings->MinimumLayerWeight))
							{
								InstancesToDelete.Add(InstanceIndex);
								if (bReapplyLocation)
								{
									// restore the location so the hash removal will succeed
									Instance.Location = OldInstanceLocation;
								}
								continue;
							}
						}
					}

					// Reapply vertex color mask
					if (Settings->ReapplyVertexColorMask)
					{
						if (Hit.FaceIndex != INDEX_NONE && IsUsingVertexColorMask(Settings))
						{
							UStaticMeshComponent* HitStaticMeshComponent = Cast<UStaticMeshComponent>(Hit.Component.Get());
							if (HitStaticMeshComponent)
							{
								FColor VertexColor;
								if (GetStaticMeshVertexColorForHit(HitStaticMeshComponent, Hit.FaceIndex, Hit.Location, VertexColor))
								{
									if (!CheckVertexColor(Settings, VertexColor))
									{
										InstancesToDelete.Add(InstanceIndex);
										if (bReapplyLocation)
										{
											// restore the location so the hash removal will succeed
											Instance.Location = OldInstanceLocation;
										}
										continue;
									}
								}
							}
						}
					}
				}
			}

			// Cull instances that don't meet the height range
			if (Settings->ReapplyHeight)
			{
				if (!HeightInterval.Contains(Instance.Location.Z))
				{
					InstancesToDelete.Add(InstanceIndex);
					if (bReapplyLocation)
					{
						// restore the location so the hash removal will succeed
						Instance.Location = OldInstanceLocation;
					}
					continue;
				}
			}

			if (bUpdated && FMath::Abs(Instance.ZOffset) > KINDA_SMALL_NUMBER)
			{
				// Reapply the Z offset in new local space
				Instance.Location = Instance.GetInstanceWorldTransform().TransformPosition(FVector(0, 0, Instance.ZOffset));
				bReapplyLocation = true;
			}

			// Update the hash
			if (bReapplyLocation)
			{
				FoliageInfo->InstanceHash->RemoveInstance(OldInstanceLocation, InstanceIndex);
				FoliageInfo->InstanceHash->InsertInstance(Instance.Location, InstanceIndex);
			}

			const float SettingsRadius = Settings->GetRadius(bSingleInstanceMode);
			// Cull overlapping based on radius
			if (Settings->ReapplyRadius && SettingsRadius > 0.f)
			{
				if (FoliageInfo->CheckForOverlappingInstanceExcluding(InstanceIndex, SettingsRadius, InstancesToDelete))
				{
					InstancesToDelete.Add(InstanceIndex);
					continue;
				}
			}

			// Remove mesh collide with world
			if (Settings->ReapplyCollisionWithWorld)
			{
				FHitResult Hit;
				static const FName NAME_ReapplyInstancesForBrush = TEXT("ReapplyCollisionWithWorld");
				FVector Start = Instance.Location + FVector(0.f, 0.f, 16.f);
				FVector End = Instance.Location - FVector(0.f, 0.f, 16.f);
				if (AInstancedFoliageActor::FoliageTrace(InWorld, Hit, FDesiredFoliageInstance(Start, End, Settings), NAME_ReapplyInstancesForBrush))
				{
					if (!AInstancedFoliageActor::CheckCollisionWithWorld(InWorld, Settings, Instance, Hit.Normal, Hit.Location, Hit.Component.Get()))
					{
						InstancesToDelete.Add(InstanceIndex);
						continue;
					}
				}
				else
				{
					InstancesToDelete.Add(InstanceIndex);
				}
			}

			if (bUpdated)
			{
				UpdatedInstances.Add(InstanceIndex);
			}
		}
	}

	if (UpdatedInstances.Num() > 0)
	{
		FoliageInfo->PostUpdateInstances(UpdatedInstances);
		IFA->RegisterAllComponents();
	}

	if (InstancesToDelete.Num())
	{
		FoliageInfo->RemoveInstances(InstancesToDelete.Array(), true);
	}
}

void FEdModeFoliage::ReapplyInstancesDensityForBrush(UWorld* InWorld, const UFoliageType* Settings, const FSphere& BrushSphere, float Pressure)
{
	if (Settings->ReapplyDensity && !FMath::IsNearlyEqual(Settings->DensityAdjustmentFactor, 1.f))
	{
		// Determine number of instances at the start of the brush stroke
		int32 SnapshotInstanceCount = 0;
		TArray<const FMeshInfoSnapshot*> SnapshotList;
		InstanceSnapshot.MultiFindPointer(const_cast<UFoliageType*>(Settings), SnapshotList);
		for (const auto* Snapshot : SnapshotList)
		{
			SnapshotInstanceCount += Snapshot->CountInstancesInsideSphere(BrushSphere);
		}

		// Determine desired number of instances
		int32 DesiredInstanceCount = FMath::RoundToInt((float)SnapshotInstanceCount * Settings->DensityAdjustmentFactor);

		if (Settings->DensityAdjustmentFactor > 1.f)
		{
			AddInstancesForBrush(InWorld, Settings, BrushSphere, DesiredInstanceCount, Pressure);
		}
		else if (Settings->DensityAdjustmentFactor < 1.f)
		{
			RemoveInstancesForBrush(InWorld, Settings, BrushSphere, DesiredInstanceCount, Pressure);
		}
	}
}

void FEdModeFoliage::PreApplyBrush()
{
	InstanceSnapshot.Empty();

	UWorld* World = GetWorld();
	// Special setup beginning a stroke with the Reapply tool
	// Necessary so we don't keep reapplying settings over and over for the same instances.
	if (UISettings.GetReapplyToolSelected())
	{
		for (auto& FoliageMeshUI : FoliageMeshList)
		{
			UFoliageType* Settings = FoliageMeshUI->Settings;

			if (!Settings->IsSelected)
			{
				continue;
			}

			for (FFoliageInfoIterator It(World, Settings); It; ++It)
			{
				FFoliageInfo* FoliageInfo = (*It);

				// Take a snapshot of all the locations
				InstanceSnapshot.Add(Settings, FMeshInfoSnapshot(FoliageInfo));

				// Clear the "FOLIAGE_Readjusted" flag
				for (auto& Instance : FoliageInfo->Instances)
				{
					Instance.Flags &= (~FOLIAGE_Readjusted);
				}
			}
		}
	}
}

void FEdModeFoliage::ApplyBrush(FEditorViewportClient* ViewportClient)
{
	FEditorViewportClient* CurrentViewportClient = StaticCast<FEditorViewportClient*>(GEditor->GetActiveViewport()->GetClient());
	if (!bBrushTraceValid || ViewportClient != CurrentViewportClient)
	{
		return;
	}

	float BrushArea = PI * FMath::Square(UISettings.GetRadius());

	// Tablet pressure or motion controller pressure
	const float Pressure = ViewportClient->Viewport->IsPenActive() ? ViewportClient->Viewport->GetTabletPressure() : 1.f;

	// Cache a copy of the world pointer
	UWorld* World = ViewportClient->GetWorld();
	int32 SelectedIndex = -1;
	TArray<FFoliageMeshUIInfoPtr> SelectedFoliageMeshList;
	SelectedFoliageMeshList.Reserve(FoliageMeshList.Num());
	for (auto& FoliageMeshUI : FoliageMeshList)
	{
		if (FoliageMeshUI->Settings->IsSelected)
		{
			SelectedFoliageMeshList.Add(FoliageMeshUI);
		}
	}

	for (int32 Index = 0; Index < SelectedFoliageMeshList.Num(); ++Index)
	{
		UFoliageType* Settings = SelectedFoliageMeshList[Index]->Settings;
		ON_SCOPE_EXIT
		{
			OnInstanceCountUpdated(Settings);
		};

		FSphere BrushSphere(BrushLocation, UISettings.GetRadius());

		if (UISettings.GetLassoSelectToolSelected())
		{
			SelectInstancesForBrush(World, Settings, BrushSphere, !IsModifierButtonPressed(ViewportClient));
		}
		else if (UISettings.GetReapplyToolSelected())
		{
			// Reapply any settings checked by the user
			ReapplyInstancesForBrush(World, Settings, BrushSphere, Pressure, UISettings.IsInAnySingleInstantiationMode());
		}
		else if (UISettings.GetPaintToolSelected())
		{
			if (UISettings.GetEraseToolSelected() || IsModifierButtonPressed(ViewportClient))
			{
				int32 DesiredInstanceCount = FMath::RoundToInt(BrushArea * Settings->Density * UISettings.GetUnpaintDensity() / (1000.f*1000.f));

				RemoveInstancesForBrush(World, Settings, BrushSphere, DesiredInstanceCount, Pressure);
			}
			else
			{
				if (UISettings.IsInAnySingleInstantiationMode())
				{
					if (UISettings.GetSingleInstantiationPlacementMode() == EFoliageSingleInstantiationPlacementMode::Type::All)
					{
						AddSingleInstanceForBrush(World, Settings, Pressure);
					}
					else if (UISettings.GetSingleInstantiationPlacementMode() == EFoliageSingleInstantiationPlacementMode::Type::CycleThrough)
					{
						if (UISettings.GetSingleInstantiationCycleThroughIndex() % SelectedFoliageMeshList.Num() == Index)
						{
							if (AddSingleInstanceForBrush(World, Settings, Pressure))
							{
								UISettings.IncrementSingleInstantiationCycleThroughIndex();
							}
							return;
						}
					}
				}
				else
				{
					// This is the total set of instances disregarding parameters like slope, height or layer.
					float DesiredInstanceCountFloat = BrushArea * Settings->Density * UISettings.GetPaintDensity() / (1000.f*1000.f);
					// Allow a single instance with a random chance, if the brush is smaller than the density
					int32 DesiredInstanceCount = DesiredInstanceCountFloat > 1.f ? FMath::RoundToInt(DesiredInstanceCountFloat) : FMath::FRand() < DesiredInstanceCountFloat ? 1 : 0;

					AddInstancesForBrush(World, Settings, BrushSphere, DesiredInstanceCount, Pressure);
				}
			}
		}
	}

	if (UISettings.GetLassoSelectToolSelected())
	{
		UpdateWidgetLocationToInstanceSelection();
	}
}

struct FFoliagePaintBucketTriangle
{
	FFoliagePaintBucketTriangle(const FTransform& InLocalToWorld, const FVector& InVertex0, const FVector& InVertex1, const FVector& InVertex2, const FColor InColor0, const FColor InColor1, const FColor InColor2)
	{
		Vertex = InLocalToWorld.TransformPosition(InVertex0);
		Vector1 = InLocalToWorld.TransformPosition(InVertex1) - Vertex;
		Vector2 = InLocalToWorld.TransformPosition(InVertex2) - Vertex;
		VertexColor[0] = InColor0;
		VertexColor[1] = InColor1;
		VertexColor[2] = InColor2;

		WorldNormal = InLocalToWorld.GetDeterminant() >= 0.f ? Vector2 ^ Vector1 : Vector1 ^ Vector2;
		FVector::FReal WorldNormalSize = WorldNormal.Size();
		Area = WorldNormalSize * 0.5f;
		if (WorldNormalSize > SMALL_NUMBER)
		{
			WorldNormal /= WorldNormalSize;
		}
	}

	void GetRandomPoint(FVector& OutPoint, FColor& OutBaryVertexColor)
	{
		// Sample parallelogram
		float x = FMath::FRand();
		float y = FMath::FRand();

		// Flip if we're outside the triangle
		if (x + y > 1.f)
		{
			x = 1.f - x;
			y = 1.f - y;
		}

		OutBaryVertexColor = ((1.f - x - y) * VertexColor[0] + x * VertexColor[1] + y * VertexColor[2]).ToFColor(true);
		OutPoint = Vertex + x * Vector1 + y * Vector2;
	}

	FVector	Vertex;
	FVector Vector1;
	FVector Vector2;
	FVector WorldNormal;
	FVector::FReal Area;
	FColor VertexColor[3];
};

/** Apply paint bucket to actor */
void FEdModeFoliage::ApplyPaintBucket_Remove(AActor* Actor)
{
	UWorld* World = Actor->GetWorld();

	TInlineComponentArray<UActorComponent*> Components;
	Actor->GetComponents(Components);

	// Remove all instances of the selected meshes
	for (const auto& MeshUIInfo : FoliageMeshList)
	{
		UFoliageType* FoliageType = MeshUIInfo->Settings;
		if (!FoliageType->IsSelected)
		{
			continue;
		}

		// Go through all FoliageActors in the world and delete 
		for (FFoliageInfoIterator It(World, FoliageType); It; ++It)
		{
			AInstancedFoliageActor* IFA = It.GetActor();

			for (auto Component : Components)
			{
				IFA->DeleteInstancesForComponent(Component, FoliageType);
			}
		}

		OnInstanceCountUpdated(FoliageType);
	}
}

/** Apply paint bucket to actor */
void FEdModeFoliage::ApplyPaintBucket_Add(AActor* Actor)
{
	UWorld* World = Actor->GetWorld();
	TMap<UPrimitiveComponent*, TArray<FFoliagePaintBucketTriangle> > ComponentPotentialTriangles;

	// Check all the components of the hit actor
	TInlineComponentArray<UStaticMeshComponent*> StaticMeshComponents;
	Actor->GetComponents(StaticMeshComponents);

	for (auto StaticMeshComponent : StaticMeshComponents)
	{
		UMaterialInterface* Material = StaticMeshComponent->GetMaterial(0);

		if (UISettings.bFilterStaticMesh && StaticMeshComponent->GetStaticMesh() && StaticMeshComponent->GetStaticMesh()->GetRenderData() &&
			(UISettings.bFilterTranslucent || !Material || !IsTranslucentBlendMode(*Material)))
		{
			UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
			FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[0];
			TArray<FFoliagePaintBucketTriangle>& PotentialTriangles = ComponentPotentialTriangles.Add(StaticMeshComponent, TArray<FFoliagePaintBucketTriangle>());

			bool bHasInstancedColorData = false;
			const FStaticMeshComponentLODInfo* InstanceMeshLODInfo = nullptr;
			if (StaticMeshComponent->LODData.Num() > 0)
			{
				InstanceMeshLODInfo = StaticMeshComponent->LODData.GetData();
				bHasInstancedColorData = InstanceMeshLODInfo->PaintedVertices.Num() == LODModel.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices();
			}

			const bool bHasColorData = bHasInstancedColorData || LODModel.VertexBuffers.ColorVertexBuffer.GetNumVertices();

			// Get the raw triangle data for this static mesh
			FTransform LocalToWorld = StaticMeshComponent->GetComponentTransform();
			FIndexArrayView Indices = LODModel.IndexBuffer.GetArrayView();
			const FPositionVertexBuffer& PositionVertexBuffer = LODModel.VertexBuffers.PositionVertexBuffer;
			const FColorVertexBuffer& ColorVertexBuffer = LODModel.VertexBuffers.ColorVertexBuffer;

			if (USplineMeshComponent* SplineMesh = Cast<USplineMeshComponent>(StaticMeshComponent))
			{
				// Transform spline mesh verts correctly
				FVector Mask = FVector(1, 1, 1);
				USplineMeshComponent::GetAxisValueRef(Mask, SplineMesh->ForwardAxis) = 0;

				for (int32 Idx = 0; Idx < Indices.Num(); Idx += 3)
				{
					const int32 Index0 = Indices[Idx];
					const int32 Index1 = Indices[Idx + 1];
					const int32 Index2 = Indices[Idx + 2];

					const FVector Vert0 = SplineMesh->CalcSliceTransform(USplineMeshComponent::GetAxisValueRef(PositionVertexBuffer.VertexPosition(Index0), SplineMesh->ForwardAxis)).TransformPosition((FVector)PositionVertexBuffer.VertexPosition(Index0) * Mask);
					const FVector Vert1 = SplineMesh->CalcSliceTransform(USplineMeshComponent::GetAxisValueRef(PositionVertexBuffer.VertexPosition(Index1), SplineMesh->ForwardAxis)).TransformPosition((FVector)PositionVertexBuffer.VertexPosition(Index1) * Mask);
					const FVector Vert2 = SplineMesh->CalcSliceTransform(USplineMeshComponent::GetAxisValueRef(PositionVertexBuffer.VertexPosition(Index2), SplineMesh->ForwardAxis)).TransformPosition((FVector)PositionVertexBuffer.VertexPosition(Index2) * Mask);

					new(PotentialTriangles)FFoliagePaintBucketTriangle(LocalToWorld
						, Vert0
						, Vert1
						, Vert2
						, bHasInstancedColorData ? InstanceMeshLODInfo->PaintedVertices[Index0].Color : (bHasColorData ? ColorVertexBuffer.VertexColor(Index0) : FColor::White)
						, bHasInstancedColorData ? InstanceMeshLODInfo->PaintedVertices[Index1].Color : (bHasColorData ? ColorVertexBuffer.VertexColor(Index1) : FColor::White)
						, bHasInstancedColorData ? InstanceMeshLODInfo->PaintedVertices[Index2].Color : (bHasColorData ? ColorVertexBuffer.VertexColor(Index2) : FColor::White)
						);
				}
			}
			else
			{
				// Build a mapping of vertex positions to vertex colors.  Using a TMap will allow for fast lookups so we can match new static mesh vertices with existing colors 
				for (int32 Idx = 0; Idx < Indices.Num(); Idx += 3)
				{
					const int32 Index0 = Indices[Idx];
					const int32 Index1 = Indices[Idx + 1];
					const int32 Index2 = Indices[Idx + 2];

					new(PotentialTriangles)FFoliagePaintBucketTriangle(LocalToWorld
						, (FVector)PositionVertexBuffer.VertexPosition(Index0)
						, (FVector)PositionVertexBuffer.VertexPosition(Index1)
						, (FVector)PositionVertexBuffer.VertexPosition(Index2)
						, bHasInstancedColorData ? InstanceMeshLODInfo->PaintedVertices[Index0].Color : (bHasColorData ? ColorVertexBuffer.VertexColor(Index0) : FColor::White)
						, bHasInstancedColorData ? InstanceMeshLODInfo->PaintedVertices[Index1].Color : (bHasColorData ? ColorVertexBuffer.VertexColor(Index1) : FColor::White)
						, bHasInstancedColorData ? InstanceMeshLODInfo->PaintedVertices[Index2].Color : (bHasColorData ? ColorVertexBuffer.VertexColor(Index2) : FColor::White)
						);
				}
			}
		}
	}

	const bool bSingleIntanceMode = UISettings.IsInAnySingleInstantiationMode();
	for (const auto& MeshUIInfo : FoliageMeshList)
	{
		UFoliageType* Settings = MeshUIInfo->Settings;
		if (!Settings->IsSelected)
		{
			continue;
		}

		// Quick lookup of potential instance locations, used for overlapping check.
		TArray<FVector> PotentialInstanceLocations;
		FFoliageInstanceHash PotentialInstanceHash(7);	// use 128x128 cell size, as the brush radius is typically small.
		TArray<FPotentialInstance> InstancesToPlace;

		for (TMap<UPrimitiveComponent*, TArray<FFoliagePaintBucketTriangle> >::TIterator ComponentIt(ComponentPotentialTriangles); ComponentIt; ++ComponentIt)
		{
			UPrimitiveComponent* Component = ComponentIt.Key();
			TArray<FFoliagePaintBucketTriangle>& PotentialTriangles = ComponentIt.Value();

			for (int32 TriIdx = 0; TriIdx<PotentialTriangles.Num(); TriIdx++)
			{
				FFoliagePaintBucketTriangle& Triangle = PotentialTriangles[TriIdx];

				// Check if we can reject this triangle based on normal.
				if (!IsWithinSlopeAngle(Triangle.WorldNormal.Z, Settings->GroundSlopeAngle.Min, Settings->GroundSlopeAngle.Max))
				{
					continue;
				}

				// This is the total set of instances disregarding parameters like slope, height or layer.
				FVector::FReal DesiredInstanceCountReal = Triangle.Area * Settings->Density * UISettings.GetPaintDensity() / (1000.f*1000.f);

				// Allow a single instance with a random chance, if the brush is smaller than the density
				int64 DesiredInstanceCount = DesiredInstanceCountReal > 1.f ? FMath::RoundToInt(DesiredInstanceCountReal) : FMath::FRand() < DesiredInstanceCountReal ? 1 : 0;

				for (int64 Idx = 0; Idx < DesiredInstanceCountReal; Idx++)
				{
					FVector InstLocation;
					FColor VertexColor;
					Triangle.GetRandomPoint(InstLocation, VertexColor);

					// Check color mask and filters at this location
					if (!CheckVertexColor(Settings, VertexColor) ||
						!CheckLocationForPotentialInstance(World, Settings, bSingleIntanceMode, InstLocation, Triangle.WorldNormal, PotentialInstanceLocations, PotentialInstanceHash))
					{
						continue;
					}

					InstancesToPlace.Emplace(FPotentialInstance(InstLocation, Triangle.WorldNormal, Component, 1.f));
				}
			}
		}

		{
			SCOPE_CYCLE_COUNTER(STAT_FoliageSpawnInstance);

			// Place instances
			TArray<FFoliageInstance> PlacedInstances;
			PlacedInstances.Reserve(InstancesToPlace.Num());

			for (FPotentialInstance& PotentialInstance : InstancesToPlace)
			{
				FFoliageInstance Inst;
				if (PotentialInstance.PlaceInstance(World, Settings, Inst))
				{
					Inst.BaseComponent = PotentialInstance.HitComponent;
					PlacedInstances.Add(MoveTemp(Inst));
				}
			}

			SpawnFoliageInstance(World, Settings, &UISettings, PlacedInstances, false);
		}

		RebuildFoliageTree(Settings);

		//
		OnInstanceCountUpdated(Settings);
	}

	CurrentFoliageTraceBrushAffectedIFAs.Empty();
}

bool FEdModeFoliage::GetStaticMeshVertexColorForHit(const UStaticMeshComponent* InStaticMeshComponent, int32 InTriangleIndex, const FVector& InHitLocation, FColor& OutVertexColor)
{
	const UStaticMesh* StaticMesh = InStaticMeshComponent->GetStaticMesh();

	if (StaticMesh == nullptr || StaticMesh->GetRenderData() == nullptr)
	{
		return false;
	}

	const FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[0];
	bool bHasInstancedColorData = false;
	const FStaticMeshComponentLODInfo* InstanceMeshLODInfo = nullptr;
	if (InStaticMeshComponent->LODData.Num() > 0)
	{
		InstanceMeshLODInfo = InStaticMeshComponent->LODData.GetData();
		bHasInstancedColorData = InstanceMeshLODInfo->PaintedVertices.Num() == LODModel.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices();
	}

	const FColorVertexBuffer& ColorVertexBuffer = LODModel.VertexBuffers.ColorVertexBuffer;

	// no vertex color data
	if (!bHasInstancedColorData && ColorVertexBuffer.GetNumVertices() == 0)
	{
		return false;
	}

	// Get the raw triangle data for this static mesh
	FIndexArrayView Indices = LODModel.IndexBuffer.GetArrayView();
	const FPositionVertexBuffer& PositionVertexBuffer = LODModel.VertexBuffers.PositionVertexBuffer;

	int32 SectionFirstTriIndex = 0;
	for (const FStaticMeshSection& Section : LODModel.Sections)
	{
		if (Section.bEnableCollision)
		{
			int32 NextSectionTriIndex = SectionFirstTriIndex + Section.NumTriangles;
			if (InTriangleIndex >= SectionFirstTriIndex && InTriangleIndex < NextSectionTriIndex)
			{

				int32 IndexBufferIdx = (InTriangleIndex - SectionFirstTriIndex) * 3 + Section.FirstIndex;

				// Look up the triangle vertex indices
				int32 Index0 = Indices[IndexBufferIdx];
				int32 Index1 = Indices[IndexBufferIdx + 1];
				int32 Index2 = Indices[IndexBufferIdx + 2];

				// Lookup the triangle world positions and colors.
				FVector WorldVert0 = InStaticMeshComponent->GetComponentTransform().TransformPosition((FVector)PositionVertexBuffer.VertexPosition(Index0));
				FVector WorldVert1 = InStaticMeshComponent->GetComponentTransform().TransformPosition((FVector)PositionVertexBuffer.VertexPosition(Index1));
				FVector WorldVert2 = InStaticMeshComponent->GetComponentTransform().TransformPosition((FVector)PositionVertexBuffer.VertexPosition(Index2));

				FLinearColor Color0;
				FLinearColor Color1;
				FLinearColor Color2;

				if (bHasInstancedColorData)
				{
					Color0 = InstanceMeshLODInfo->PaintedVertices[Index0].Color.ReinterpretAsLinear();
					Color1 = InstanceMeshLODInfo->PaintedVertices[Index1].Color.ReinterpretAsLinear();
					Color2 = InstanceMeshLODInfo->PaintedVertices[Index2].Color.ReinterpretAsLinear();
				}
				else
				{
					Color0 = ColorVertexBuffer.VertexColor(Index0).ReinterpretAsLinear();
					Color1 = ColorVertexBuffer.VertexColor(Index1).ReinterpretAsLinear();
					Color2 = ColorVertexBuffer.VertexColor(Index2).ReinterpretAsLinear();
				}

				// find the barycentric coordiantes of the hit location, so we can interpolate the vertex colors
				FVector3f BaryCoords = (FVector3f)FMath::GetBaryCentric2D(InHitLocation, WorldVert0, WorldVert1, WorldVert2);

				FLinearColor InterpColor = BaryCoords.X * Color0 + BaryCoords.Y * Color1 + BaryCoords.Z * Color2;

				// convert back to FColor.
				OutVertexColor = InterpColor.ToFColor(false);

				return true;
			}

			SectionFirstTriIndex = NextSectionTriIndex;
		}
	}

	return false;
}

void FEdModeFoliage::SnapSelectedInstancesToGround(UWorld* InWorld)
{
	FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "FoliageMode_Transaction_SnapToGround", "Snap Foliage To Ground"));
	{
		bool bMovedInstance = false;

		for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
		{
			AInstancedFoliageActor* IFA = *It;
			bool bFoundSelection = false;

			IFA->ForEachFoliageInfo([this, IFA, &bFoundSelection, &bMovedInstance](UFoliageType* FoliageType, FFoliageInfo& FoliageInfo)
			{
				TArray<int32> SelectedIndices = FoliageInfo.SelectedIndices.Array();

				if (SelectedIndices.Num() > 0)
				{
					// Mark actor once we found selection
					if (!bFoundSelection)
					{
						IFA->Modify();
						bFoundSelection = true;
					}

					TArray<int32> MovingInstances = FoliageInfo.SelectedIndices.Intersect(FoliageInfo.MovingInstances).Array();
					TArray<int32> NonMovingInstances = FoliageInfo.SelectedIndices.Difference(FoliageInfo.MovingInstances).Array();

					// Call PostMove on already moving instances to add them back to the hash.
					if (MovingInstances.Num())
					{
						check(bMoving);
						FoliageInfo.PostMoveInstances(MovingInstances, /*bFinished=*/false);
					}

					// Call PreMove on all snapping instances.
					FoliageInfo.PreMoveInstances(SelectedIndices);

					for (int32 InstanceIndex : SelectedIndices)
					{
						bMovedInstance |= SnapInstanceToGround(IFA, FoliageType, FoliageInfo, InstanceIndex);
					}

					if (MovingInstances.Num())
					{
						FoliageInfo.PostMoveInstances(MovingInstances, /*bFinished=*/false);
						FoliageInfo.PreMoveInstances(MovingInstances);
					}

					if (NonMovingInstances.Num())
					{
						FoliageInfo.PostMoveInstances(NonMovingInstances, /*bFinished=*/true);
					}
				}
				return true; // continue iteration
			});
		}

		if (bMovedInstance)
		{
			UpdateWidgetLocationToInstanceSelection();
		}
	}
}

void FEdModeFoliage::HandleOnActorSpawned(AActor* Actor)
{
	if (AInstancedFoliageActor* IFA = Cast<AInstancedFoliageActor>(Actor))
	{
		// If an IFA was created, we want to be notified if any meshes assigned to its foliage types change
		IFA->OnFoliageTypeMeshChanged().AddSP(this, &FEdModeFoliage::HandleOnFoliageTypeMeshChanged);
	}
}

void FEdModeFoliage::HandleOnFoliageTypeMeshChanged(UFoliageType* FoliageType)
{
	if (FoliageType->IsNotAssetOrBlueprint() && FoliageType->GetSource() == nullptr)
	{
		RemoveFoliageType(&FoliageType, 1);
	}
	else
	{
		StaticCastSharedPtr<FFoliageEdModeToolkit>(Toolkit)->NotifyFoliageTypeMeshChanged(FoliageType);
	}
}

bool FEdModeFoliage::SnapInstanceToGround(AInstancedFoliageActor* InIFA, const UFoliageType* Settings, FFoliageInfo& Mesh, int32 InstanceIdx)
{
	FFoliageInstance& Instance = Mesh.Instances[InstanceIdx];
	FVector Start = Instance.Location;
	FVector End = Instance.Location - FVector(0.f, 0.f, FOLIAGE_SNAP_TRACE);

	FHitResult Hit;
	static FName NAME_FoliageSnap = FName("FoliageSnap");
	if (AInstancedFoliageActor::FoliageTrace(InIFA->GetWorld(), Hit, FDesiredFoliageInstance(Start, End, Settings), NAME_FoliageSnap, /* bReturnFaceIndex */ false, FFoliageTraceFilterFunc(), (Instance.Flags & FOLIAGE_AlignToNormal)))
	{
		UPrimitiveComponent* HitComponent = Hit.Component.Get();

		if (HitComponent->GetComponentLevel() != InIFA->GetLevel())
		{
			// We should not create cross-level references automatically
			return false;
		}

		// We cannot be based on an a blueprint component as these will disappear when the construction script is re-run
		if (HitComponent->IsCreatedByConstructionScript())
		{
			return false;
		}

		// Find BSP brush 
		UModelComponent* ModelComponent = Cast<UModelComponent>(HitComponent);
		if (ModelComponent)
		{
			ABrush* BrushActor = ModelComponent->GetModel()->FindBrush((FVector3f)Hit.Location);
			if (BrushActor)
			{
				HitComponent = BrushActor->GetBrushComponent();
			}
		}

		// Set new base
		auto NewBaseId = InIFA->InstanceBaseCache.AddInstanceBaseId(Mesh.ShouldAttachToBaseComponent() ? HitComponent : nullptr);
		Mesh.RemoveFromBaseHash(InstanceIdx);
		Instance.BaseId = NewBaseId;
		if (Instance.BaseId == FFoliageInstanceBaseCache::InvalidBaseId)
		{
			Instance.BaseComponent = nullptr;
		}
		Mesh.AddToBaseHash(InstanceIdx);
		Instance.Location = Hit.Location;
		Instance.ZOffset = 0.f;

		if (Instance.Flags & FOLIAGE_AlignToNormal)
		{
			// Remove previous alignment and align to new normal.
			Instance.Rotation = Instance.PreAlignRotation;
			Instance.AlignToNormal(Hit.Normal, Settings->AlignMaxAngle);
		}

		return true;
	}

	return false;
}

TArray<FFoliageMeshUIInfoPtr>& FEdModeFoliage::GetFoliageMeshList()
{
	return FoliageMeshList;
}

void FEdModeFoliage::PopulateFoliageMeshList()
{
	FoliageMeshList.Empty();

	// Collect set of all available foliage types
	UWorld* World = GEditor->GetEditorWorldContext().World();
	ULevel* CurrentLevel = World->GetCurrentLevel();

	for (TActorIterator<AInstancedFoliageActor> It(World); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		for (const auto& MeshPair : IFA->GetFoliageInfos())
		{
			//@todo_ow: this doesn't make sense in WP
			if (!CanPaint(MeshPair.Key, CurrentLevel))
			{
				continue;
			}

			if (!IAssetTools::Get().IsAssetVisible(FAssetData(MeshPair.Key)))
			{
				continue;
			}

			int32 ElementIdx = FoliageMeshList.IndexOfByPredicate([&](const FFoliageMeshUIInfoPtr& Item)
			{
				return Item->Settings == MeshPair.Key;
			});

			if (ElementIdx == INDEX_NONE)
			{
				ElementIdx = FoliageMeshList.Add(MakeShareable(new FFoliageMeshUIInfo(MeshPair.Key)));
			}

			int32 PlacedInstanceCount = MeshPair.Value->GetPlacedInstanceCount();
			FoliageMeshList[ElementIdx]->InstanceCountTotal += PlacedInstanceCount;

			//@todo_ow: this doesn't make sense in WP
			if (IFA->GetLevel() == World->GetCurrentLevel())
			{
				FoliageMeshList[ElementIdx]->InstanceCountCurrentLevel += PlacedInstanceCount;
			}
		}
	}

	if (FoliageMeshListSortMode != EColumnSortMode::None)
	{
		EColumnSortMode::Type SortMode = FoliageMeshListSortMode;
		auto CompareFoliageType = [SortMode](const FFoliageMeshUIInfoPtr& A, const FFoliageMeshUIInfoPtr& B)
		{
			bool CompareResult = (A->GetNameText().CompareToCaseIgnored(B->GetNameText()) <= 0);
			return SortMode == EColumnSortMode::Ascending ? CompareResult : !CompareResult;
		};

		FoliageMeshList.Sort(CompareFoliageType);
	}

	StaticCastSharedPtr<FFoliageEdModeToolkit>(Toolkit)->RefreshFullList();
}

void FEdModeFoliage::OnFoliageMeshListSortModeChanged(EColumnSortMode::Type InSortMode)
{
	FoliageMeshListSortMode = InSortMode;
	PopulateFoliageMeshList();
}

EColumnSortMode::Type FEdModeFoliage::GetFoliageMeshListSortMode() const
{
	return FoliageMeshListSortMode;
}

void FEdModeFoliage::OnInstanceCountUpdated(const UFoliageType* FoliageType)
{
	int32 EntryIndex = FoliageMeshList.IndexOfByPredicate([&](const FFoliageMeshUIInfoPtr& UIInfoPtr)
	{
		return UIInfoPtr->Settings == FoliageType;
	});

	if (EntryIndex == INDEX_NONE)
	{
		return;
	}

	int32 InstanceCountTotal = 0;
	int32 InstanceCountCurrentLevel = 0;
	UWorld* World = GetWorld();
	ULevel* CurrentLevel = World->GetCurrentLevel();

	for (FFoliageInfoIterator It(World, FoliageType); It; ++It)
	{
		FFoliageInfo* FoliageInfo = (*It);
		InstanceCountTotal += FoliageInfo->Instances.Num();
		if (It.GetActor()->GetLevel() == CurrentLevel)
		{
			InstanceCountCurrentLevel += FoliageInfo->Instances.Num();
		}
	}

	//
	FoliageMeshList[EntryIndex]->InstanceCountTotal = InstanceCountTotal;
	FoliageMeshList[EntryIndex]->InstanceCountCurrentLevel = InstanceCountCurrentLevel;
}

void FEdModeFoliage::CalcTotalInstanceCount(int32& OutInstanceCountTotal, int32& OutInstanceCountCurrentLevel)
{
	OutInstanceCountTotal = 0;
	OutInstanceCountCurrentLevel = 0;
	UWorld* InWorld = GetWorld();
	ULevel* CurrentLevel = InWorld->GetCurrentLevel();
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		int32 IFAInstanceCount = 0;
		for (const auto& MeshPair : IFA->GetFoliageInfos())
		{
			const FFoliageInfo& FoliageInfo = *MeshPair.Value;
			IFAInstanceCount += FoliageInfo.Instances.Num();
		}

		OutInstanceCountTotal += IFAInstanceCount;
		//@todo_ow: This doesn't make much sense in World Partition
		if (CurrentLevel == IFA->GetLevel())
		{
			OutInstanceCountCurrentLevel += IFAInstanceCount;
		}
	}
}

bool FEdModeFoliage::CanPaint(const ULevel* InLevel)
{
	for (const auto& MeshUIPtr : FoliageMeshList)
	{
		if (MeshUIPtr->Settings->IsSelected && CanPaint(MeshUIPtr->Settings, InLevel))
		{
			return true;
		}
	}

	return false;
}

bool FEdModeFoliage::CanPaint(const UFoliageType* FoliageType, const ULevel* InLevel)
{
	if (FoliageType == nullptr)	//if asset has already been deleted we can't paint
	{
		return false;
	}

	// Non-shared objects can be painted only into their own level
	// Assets can be painted everywhere
	if (FoliageType->IsAsset() || FoliageType->GetTypedOuter<ULevel>() == InLevel)
	{
		return true;
	}

	return false;
}

bool FEdModeFoliage::IsModifierButtonPressed(const FEditorViewportClient* ViewportClient) const
{
	return IsShiftDown(ViewportClient->Viewport);
}

void FEdModeFoliage::MoveSelectedFoliageToActorEditorContext()
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MoveSelectedFoliageToActorEditorContext", "Move Selected Foliage to Actor Editor Context"));
	AInstancedFoliageActor::MoveSelectedInstancesToActorEditorContext(GetWorld());
}

bool FEdModeFoliage::CanMoveSelectedFoliageToLevel(ULevel* InTargetLevel) const
{
	UWorld* World = InTargetLevel->OwningWorld;
	const int32 NumLevels = World->GetNumLevels();

	for (int32 LevelIdx = 0; LevelIdx < NumLevels; ++LevelIdx)
	{
		ULevel* Level = World->GetLevel(LevelIdx);
		if (Level != InTargetLevel)
		{
			AInstancedFoliageActor* IFA = AInstancedFoliageActor::GetInstancedFoliageActorForLevel(Level, /*bCreateIfNone*/ false);

			if (IFA && IFA->HasSelectedInstances())
			{
				return true;
			}
		}
	}

	return false;
}

void FEdModeFoliage::MoveSelectedFoliageToLevel(ULevel* InTargetLevel)
{
	// Can't move into a locked level
	if (FLevelUtils::IsLevelLocked(InTargetLevel))
	{
		FNotificationInfo NotificatioInfo(NSLOCTEXT("UnrealEd", "CannotMoveFoliageIntoLockedLevel", "Cannot move the selected foliage into a locked level"));
		NotificatioInfo.bUseThrobber = false;
		FSlateNotificationManager::Get().AddNotification(NotificatioInfo)->SetCompletionState(SNotificationItem::CS_Fail);
		return;
	}

	// Get a world context
	UWorld* World = InTargetLevel->OwningWorld;
	bool PromptToMoveFoliageTypeToAsset = World->GetStreamingLevels().Num() > 0;
	bool ShouldPopulateMeshList = false;

	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MoveSelectedFoliageToSelectedLevel", "Move Selected Foliage to Level"));
	
	FEdModeFoliageSelectionUpdate Scope(this);
	// Iterate over all foliage actors in the world and move selected instances to a foliage actor in the target level
	const int32 NumLevels = World->GetNumLevels();
	for (int32 LevelIdx = 0; LevelIdx < NumLevels; ++LevelIdx)
	{
		ULevel* Level = World->GetLevel(LevelIdx);
		if (Level != InTargetLevel)
		{
			AInstancedFoliageActor* IFA = AInstancedFoliageActor::GetInstancedFoliageActorForLevel(Level, /*bCreateIfNone*/ false);

			if (IFA && IFA->HasSelectedInstances())
			{
				bool CanMoveInstanceType = true;

				// Make sure all our foliage type used by our selected instances are asset otherwise promote them to assets
				TMap<UFoliageType*, FFoliageInfo*> SelectedInstanceFoliageTypes = IFA->GetSelectedInstancesFoliageType();

				for (auto& MeshPair : SelectedInstanceFoliageTypes)
				{
					if (!MeshPair.Key->IsAsset())
					{
						// Keep previous selection
						TSet<int32> PreviousSelectionSet = MeshPair.Value->SelectedIndices;
						TArray<int32> PreviousSelectionArray;
						PreviousSelectionArray.Reserve(PreviousSelectionSet.Num());

						for (int32& Value : PreviousSelectionSet)
						{
							PreviousSelectionArray.Add(Value);
						}

						UFoliageType* NewFoliageType = SaveFoliageTypeObject(MeshPair.Key);
						CanMoveInstanceType = NewFoliageType != nullptr;

						if (NewFoliageType != nullptr)
						{
							// Restore previous selection for move operation
							FFoliageInfo* FoliageInfo = IFA->FindInfo(NewFoliageType);
							FoliageInfo->SelectInstances(true, PreviousSelectionArray);
						}
					}
				}
			
				// Update our actor if we saved some foliage type as asset
				if (CanMoveInstanceType)
				{
					IFA = AInstancedFoliageActor::GetInstancedFoliageActorForLevel(Level, /*bCreateIfNone*/ false);
					ensure(IFA != nullptr && IFA->HasSelectedInstances());

					IFA->MoveSelectedInstancesToLevel(InTargetLevel);
					ShouldPopulateMeshList = true;
				}
			}
		}
	}

	// Update foliage usages
	if (ShouldPopulateMeshList)
	{
		PopulateFoliageMeshList();
	}
}

UFoliageType* FEdModeFoliage::AddFoliageAsset(UObject* InAsset, bool bInPlaceholderAsset)
{
	UFoliageType* FoliageType = nullptr;

	UStaticMesh* StaticMesh = Cast<UStaticMesh>(InAsset);
	if (StaticMesh)
	{
		AInstancedFoliageActor* IFA = AInstancedFoliageActor::GetDefault(GetWorld());
		FoliageType = IFA->GetLocalFoliageTypeForSource(StaticMesh);
		if (!FoliageType)
		{
			{
				const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "FoliageMode_AddTypeTransaction", "Add Foliage Type"));
				UFoliageType_InstancedStaticMesh *InstancedMeshFoliageType = NewObject<UFoliageType_InstancedStaticMesh>(IFA, NAME_None, RF_Transactional);
				InstancedMeshFoliageType->SetStaticMesh(StaticMesh);
				FoliageType = InstancedMeshFoliageType;
			}

			if (GetWorld()->GetWorldPartition()
				|| GetWorld()->GetStreamingLevels().Num() > 0)
			{
				// If the world is partitioned or the world has sublevels,
				// require the user to save the new foliage as an asset.
				// The user will then be able to paint over all cells/sub-level.
				FoliageType = SaveFoliageTypeObject(FoliageType, bInPlaceholderAsset);
			}

			if (FoliageType != nullptr)
			{
				IFA->AddMesh(FoliageType);
			}
		}
	}
	else
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "FoliageMode_AddTypeTransaction", "Add Foliage Type"));

		FoliageType = Cast<UFoliageType>(InAsset);
		if (FoliageType)
		{
			AInstancedFoliageActor* IFA = AInstancedFoliageActor::GetDefault(GetWorld());
			FoliageType = IFA->AddFoliageType(FoliageType);
		}
	}

	if (FoliageType)
	{
		PopulateFoliageMeshList();
	}

	return FoliageType;
}

/** Remove a mesh */
bool FEdModeFoliage::RemoveFoliageType(UFoliageType** FoliageTypeList, int32 Num)
{
	TArray<AInstancedFoliageActor*> IFAList;
	// Find all foliage actors that have any of these types
	UWorld* World = GetWorld();
	for (int32 FoliageTypeIdx = 0; FoliageTypeIdx < Num; ++FoliageTypeIdx)
	{
		UFoliageType* FoliageType = FoliageTypeList[FoliageTypeIdx];
		for (FFoliageInfoIterator It(World, FoliageType); It; ++It)
		{
			IFAList.Add(It.GetActor());
		}
	}

	if (IFAList.Num())
	{
		{	// Transaction scope
			FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "FoliageMode_RemoveMeshTransaction", "Foliage Editing: Remove Mesh"));
			for (AInstancedFoliageActor* IFA : IFAList)
			{
				IFA->RemoveFoliageType(FoliageTypeList, Num);
			}
		}
		PopulateFoliageMeshList();
		return true;
	}

	return false;
}

/** Bake instances to StaticMeshActors */
void FEdModeFoliage::BakeFoliage(UFoliageType* Settings, bool bSelectedOnly)
{
	AInstancedFoliageActor* IFA = AInstancedFoliageActor::GetInstancedFoliageActorForCurrentLevel(GetWorld());
	if (IFA == nullptr)
	{
		return;
	}

	FFoliageInfo* FoliageInfo = IFA->FindInfo(Settings);
	if (FoliageInfo != nullptr && FoliageInfo->Type == EFoliageImplType::StaticMesh)
	{
		TArray<int32> InstancesToConvert;
		if (bSelectedOnly)
		{
			InstancesToConvert = FoliageInfo->SelectedIndices.Array();
		}
		else
		{
			for (int32 InstanceIdx = 0; InstanceIdx < FoliageInfo->Instances.Num(); InstanceIdx++)
			{
				InstancesToConvert.Add(InstanceIdx);
			}
		}

		// Convert
		for (int32 Idx = 0; Idx < InstancesToConvert.Num(); Idx++)
		{
			FFoliageInstance& Instance = FoliageInfo->Instances[InstancesToConvert[Idx]];
			// We need a world in which to spawn the actor. Use the one from the original instance.
			UWorld* World = IFA->GetWorld();
			check(World != nullptr);
			AStaticMeshActor* SMA = World->SpawnActor<AStaticMeshActor>(Instance.Location, Instance.Rotation);
			SMA->GetStaticMeshComponent()->SetStaticMesh(Cast<UStaticMesh>(Settings->GetSource()));
			SMA->GetRootComponent()->SetRelativeScale3D((FVector)Instance.DrawScale3D);
			SMA->MarkComponentsRenderStateDirty();
		}

		// Remove
		FoliageInfo->RemoveInstances(InstancesToConvert, true);
	}
}

/** Copy the settings object for this static mesh */
UFoliageType* FEdModeFoliage::CopySettingsObject(UFoliageType* Settings)
{
	FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "FoliageMode_DuplicateSettingsObject", "Foliage Editing: Copy Settings Object"));

	AInstancedFoliageActor* IFA = AInstancedFoliageActor::GetInstancedFoliageActorForCurrentLevel(GetWorld());
	IFA->Modify();

	TUniqueObj<FFoliageInfo> FoliageInfo;
	if (IFA->RemoveFoliageInfoAndCopyValue(Settings, FoliageInfo))
	{
		Settings = (UFoliageType*)StaticDuplicateObject(Settings, IFA, NAME_None, RF_AllFlags & ~(RF_Standalone | RF_Public));
		IFA->AddFoliageInfo(Settings, MoveTemp(FoliageInfo));
		return Settings;
	}
	else
	{
		Transaction.Cancel();
	}

	return nullptr;
}

/** Replace the settings object for this static mesh with the one specified */
void FEdModeFoliage::ReplaceSettingsObject(UFoliageType* OldSettings, UFoliageType* NewSettings)
{
	FFoliageEditUtility::ReplaceFoliageTypeObject(GetWorld(), OldSettings, NewSettings);

	PopulateFoliageMeshList();
}

UFoliageType* FEdModeFoliage::SaveFoliageTypeObject(UFoliageType* InFoliageType, bool bInPlaceholderAsset)
{
	UFoliageType* TypeToSave = FFoliageEditUtility::SaveFoliageTypeObject(InFoliageType, bInPlaceholderAsset);

	if (TypeToSave != nullptr && TypeToSave != InFoliageType)
	{
		ReplaceSettingsObject(InFoliageType, TypeToSave);
	}

	return TypeToSave;
}

/** Reapply cluster settings to all the instances */
void FEdModeFoliage::ReallocateClusters(UFoliageType* Settings)
{
	UWorld* World = GetWorld();
	for (FFoliageInfoIterator It(World, Settings); It; ++It)
	{
		FFoliageInfo* FoliageInfo = (*It);
		FoliageInfo->ReallocateClusters(Settings);
	}
}

/** FEdMode: Called when a key is pressed */
bool FEdModeFoliage::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	if (!IsEditingEnabled())
	{
		return false;
	}

	if (Event != IE_Released)
	{
		if (UICommandList->ProcessCommandBindings(Key, FSlateApplication::Get().GetModifierKeys(), false/*Event == IE_Repeat*/))
		{
			return true;
		}
	}

	bool bHandled = false;
	if ((UISettings.GetPaintToolSelected() || UISettings.GetReapplyToolSelected() || UISettings.GetLassoSelectToolSelected()))
	{
		// Require Ctrl or not as per user preference
		ELandscapeFoliageEditorControlType FoliageEditorControlType = GetDefault<ULevelEditorViewportSettings>()->FoliageEditorControlType;

		if (Key == EKeys::LeftMouseButton && Event == IE_Pressed)
		{
			// Only activate tool if we're not already moving the camera and we're not trying to drag a transform widget
			// Not using "if (!ViewportClient->IsMovingCamera())" because it's wrong in ortho viewports :D
			bool bMovingCamera = Viewport->KeyState(EKeys::MiddleMouseButton) || Viewport->KeyState(EKeys::RightMouseButton) || IsAltDown(Viewport);

			if ((Viewport->IsPenActive() && Viewport->GetTabletPressure() > 0.f) ||
				(!bMovingCamera && ViewportClient->GetCurrentWidgetAxis() == EAxisList::None &&
					(FoliageEditorControlType == ELandscapeFoliageEditorControlType::IgnoreCtrl ||
						(FoliageEditorControlType == ELandscapeFoliageEditorControlType::RequireCtrl   && IsCtrlDown(Viewport)) ||
						(FoliageEditorControlType == ELandscapeFoliageEditorControlType::RequireNoCtrl && !IsCtrlDown(Viewport)))))
			{
				if (!bToolActive)
				{
					StartFoliageBrushTrace(ViewportClient);

					bHandled = true;
				}
			}
		}
		else if (bToolActive && Event == IE_Released &&
			(Key == EKeys::LeftMouseButton || (FoliageEditorControlType == ELandscapeFoliageEditorControlType::RequireCtrl && (Key == EKeys::LeftControl || Key == EKeys::RightControl))))
		{
			//Set the cursor position to that of the slate cursor so it wont snap back
			Viewport->SetPreCaptureMousePosFromSlateCursor();
			EndFoliageBrushTrace();

			bHandled = true;
		}
		else if (Key == EKeys::I && Event == IE_Released)
		{
			UISettings.SetIsInQuickSingleInstantiationMode(false);
		}
		else if (Key == EKeys::I && Event == IE_Pressed)
		{
			UISettings.SetIsInQuickSingleInstantiationMode(true);
		}
		else if ((Key == EKeys::LeftShift || Key == EKeys::RightShift) && Event == IE_Released)
		{
			UISettings.SetIsInQuickEraseMode(false);
		}
		else if ((Key == EKeys::LeftShift || Key == EKeys::RightShift) && Event == IE_Pressed)
		{
			UISettings.SetIsInQuickEraseMode(true);
		}

		if (IsCtrlDown(Viewport))
		{
			// Control + scroll adjusts the brush radius
			if (Key == EKeys::MouseScrollUp)
			{
				AdjustBrushRadius(1);

				bHandled = true;
			}
			else if (Key == EKeys::MouseScrollDown)
			{
				AdjustBrushRadius(-1);

				bHandled = true;
			}
		}
	}

	if (!bHandled && (UISettings.GetLassoSelectToolSelected() || UISettings.GetSelectToolSelected()))
	{
		if (Event == IE_Pressed)
		{
			if (Key == EKeys::Platform_Delete)
			{
				RemoveSelectedInstances(GetWorld());

				bHandled = true;
			}
			else if (Key == EKeys::End)
			{
				SnapSelectedInstancesToGround(GetWorld());

				bHandled = true;
			}
		}
	}

	if (!bHandled)
	{
		if (Event == IE_Pressed)
		{
			if (Key == EKeys::F)
			{
				FocusSelectedInstances();
				bHandled = true;
			}
		}
	}

	return bHandled;
}

/** FEdMode: Render the foliage edit mode */
void FEdModeFoliage::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	/** Call parent implementation */
	FEdMode::Render(View, Viewport, PDI);
}


/** FEdMode: Render HUD elements for this tool */
void FEdModeFoliage::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
}

/** FEdMode: Check to see if an actor can be selected in this mode - no side effects */
bool FEdModeFoliage::IsSelectionAllowed(AActor* InActor, bool bInSelection) const
{
	return FFoliageHelper::IsOwnedByFoliage(InActor);
}

/** FEdMode: Handling SelectActor */
bool FEdModeFoliage::Select(AActor* InActor, bool bInSelected)
{
	return false;
}

/** FEdMode: Called when the currently selected actor has changed */
void FEdModeFoliage::ActorSelectionChangeNotify()
{
}


/** Forces real-time perspective viewports */
void FEdModeFoliage::ForceRealTimeViewports(const bool bEnable)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr< IAssetViewport > ViewportWindow = LevelEditorModule.GetFirstActiveViewport();
	if (ViewportWindow.IsValid())
	{
		FEditorViewportClient &Viewport = ViewportWindow->GetAssetViewportClient();
		if (Viewport.IsPerspective())
		{
			const FText SystemDisplayName = LOCTEXT("RealtimeOverrideMessage_Foliage", "Foliage Mode");
			if (bEnable)
			{
				const bool bShouldBeRealtime = true;
				Viewport.AddRealtimeOverride(bShouldBeRealtime, SystemDisplayName);
			}
			else
			{
				Viewport.RemoveRealtimeOverride(SystemDisplayName, false);
			}
		}
	}
}

bool FEdModeFoliage::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy *HitProxy, const FViewportClick &Click)
{
	if (!IsEditingEnabled())
	{
		return false;
	}

	if (UISettings.GetSelectToolSelected())
	{
		// If we are moving we can't change the selection. HandleClick can be called in some cases where the input delta is very small.
		// The condition in FEdModeFoliage::InputDelta where we test that drag is nearly zero should prevent this.
		if (bMoving)
		{
			return true;
		}

		if (HitProxy && HitProxy->IsA(HInstancedStaticMeshInstance::StaticGetType()))
		{
			HInstancedStaticMeshInstance* SMIProxy = ((HInstancedStaticMeshInstance*)HitProxy);
			AInstancedFoliageActor* IFA = Cast<AInstancedFoliageActor>(SMIProxy->Component->GetOwner());
			if (IFA)
			{
				IFA->SelectInstance(SMIProxy->Component, SMIProxy->InstanceIndex, Click.IsControlDown());
				// Update pivot
				UpdateWidgetLocationToInstanceSelection();
			}
		}
		else if (HitProxy && HitProxy->IsA(HActor::StaticGetType()) && FFoliageHelper::IsOwnedByFoliage(((HActor*)HitProxy)->Actor))
		{
			HActor* ActorProxy = ((HActor*)HitProxy);
			for (TActorIterator<AInstancedFoliageActor> It(GetWorld()); It; ++It)
			{
				if ((*It)->SelectInstance(ActorProxy->Actor, Click.IsControlDown()))
				{
					UpdateWidgetLocationToInstanceSelection();
					break;
				}
			}
		}
		else
		{
			if (!Click.IsControlDown())
			{
				// Select none if not trying to toggle
				SelectInstances(GetWorld(), false);
			}
		}

		return true;
	}
	else if (UISettings.GetPaintBucketToolSelected() || UISettings.GetReapplyPaintBucketToolSelected())
	{
		if (HitProxy && HitProxy->IsA(HActor::StaticGetType()))
		{
			FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "FoliageMode_EditTransaction", "Foliage Editing"));
			
			if (IsModifierButtonPressed(InViewportClient))
			{
				ApplyPaintBucket_Remove(((HActor*)HitProxy)->Actor);
			}
			else
			{
				ApplyPaintBucket_Add(((HActor*)HitProxy)->Actor);
			}
		}

		return true;
	}


	return FEdMode::HandleClick(InViewportClient, HitProxy, Click);
}

FVector FEdModeFoliage::GetWidgetLocation() const
{
	return FEdMode::GetWidgetLocation();
}


/** FEdMode: Called when a mouse button is pressed */
bool FEdModeFoliage::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	bTracking = true;
	if (UISettings.GetSelectToolSelected() || UISettings.GetLassoSelectToolSelected())
	{
		// Update pivot
		UpdateWidgetLocationToInstanceSelection();

		GEditor->BeginTransaction(NSLOCTEXT("UnrealEd", "FoliageMode_EditTransaction", "Foliage Editing"));
		
		bCanAltDrag = true;
	}
	else
	{	
		bTracking = false;
		return FEdMode::StartTracking(InViewportClient, InViewport);
	}

	return true;
}

bool FEdModeFoliage::EndTracking()
{
	bTracking = false;

	if (UISettings.GetSelectToolSelected() || UISettings.GetLassoSelectToolSelected())
	{
		PostTransformSelectedInstances(GetWorld());
		GEditor->EndTransaction();
		return true;
	}
		
	return false;
}

/** FEdMode: Called when the a mouse button is released */
bool FEdModeFoliage::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if (EndTracking())
	{
		return true;
	}

	return FEdMode::EndTracking(InViewportClient, InViewport);
}

/** FEdMode: Called when mouse drag input it applied */
bool FEdModeFoliage::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	if (InViewportClient->GetCurrentWidgetAxis() != EAxisList::None && (UISettings.GetSelectToolSelected() || UISettings.GetLassoSelectToolSelected()))
	{
		// It is possible to receive a zero delta from FEditorViewportClient::UpdateMouseDelta which can have a non zero DragDelta which gets converted to zero drag/rot/scale
		// In which case we want to avoid starting a move in case we then receive a HandleClick in the same frame
		if (!InDrag.IsNearlyZero() || !InRot.IsNearlyZero() || !InScale.IsNearlyZero())
		{
			const bool bDuplicateInstances = (bCanAltDrag && IsAltDown(InViewport) && (InViewportClient->GetCurrentWidgetAxis() & EAxisList::XYZ));

			TransformSelectedInstances(GetWorld(), InDrag, InRot, InScale, bDuplicateInstances);

			// Only allow alt-drag on first InputDelta
			bCanAltDrag = false;

			return true;
		}
	}

	return FEdMode::InputDelta(InViewportClient, InViewport, InDrag, InRot, InScale);
}

bool FEdModeFoliage::AllowWidgetMove()
{
	return ShouldDrawWidget();
}

bool FEdModeFoliage::UsesTransformWidget() const
{
	return ShouldDrawWidget();
}

bool FEdModeFoliage::ShouldDrawWidget() const
{
	if (UISettings.GetSelectToolSelected() || (UISettings.GetLassoSelectToolSelected() && !bToolActive))
	{
		FVector Location;
		return GetSelectionLocation(GetWorld(), Location);
	}
	return false;
}

EAxisList::Type FEdModeFoliage::GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const
{
	switch (InWidgetMode)
	{
	case UE::Widget::WM_Translate:
	case UE::Widget::WM_Rotate:
	case UE::Widget::WM_Scale:
		return EAxisList::XYZ;
	default:
		return EAxisList::None;
	}
}

float FEdModeFoliage::GetPaintingBrushRadius() const
{
	float Radius = UISettings.GetRadius();
	const bool bSingleInstanceMode = UISettings.IsInAnySingleInstantiationMode();
	if (bSingleInstanceMode)
	{
		for (auto& FoliageMeshUI : FoliageMeshList)
		{
			UFoliageType* Settings = FoliageMeshUI->Settings;
			if (Settings->IsSelected)
			{
				Radius = FMath::Max(Radius, Settings->GetRadius(bSingleInstanceMode));
			}
		}
	}
	return Radius;
}

/** Load UI settings from ini file */
void FFoliageUISettings::Load()
{
	FString WindowPositionString;
	if (GConfig->GetString(TEXT("FoliageEdit"), TEXT("WindowPosition"), WindowPositionString, GEditorPerProjectIni))
	{
		TArray<FString> PositionValues;
		if (WindowPositionString.ParseIntoArray(PositionValues, TEXT(","), true) == 4)
		{
			WindowX = FCString::Atoi(*PositionValues[0]);
			WindowY = FCString::Atoi(*PositionValues[1]);
			WindowWidth = FCString::Atoi(*PositionValues[2]);
			WindowHeight = FCString::Atoi(*PositionValues[3]);
		}
	}

	GConfig->GetFloat(TEXT("FoliageEdit"), TEXT("Radius"), Radius, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("FoliageEdit"), TEXT("PaintDensity"), PaintDensity, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("FoliageEdit"), TEXT("UnpaintDensity"), UnpaintDensity, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("FoliageEdit"), TEXT("bFilterLandscape"), bFilterLandscape, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("FoliageEdit"), TEXT("bFilterStaticMesh"), bFilterStaticMesh, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("FoliageEdit"), TEXT("bFilterBSP"), bFilterBSP, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("FoliageEdit"), TEXT("bFilterFoliage"), bFilterFoliage, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("FoliageEdit"), TEXT("bFilterTranslucent"), bFilterTranslucent, GEditorPerProjectIni);

	GConfig->GetBool(TEXT("FoliageEdit"), TEXT("bShowPaletteItemDetails"), bShowPaletteItemDetails, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("FoliageEdit"), TEXT("bShowPaletteItemTooltips"), bShowPaletteItemTooltips, GEditorPerProjectIni);

	int32 ActivePaletteViewModeAsInt = 0;
	GConfig->GetInt(TEXT("FoliageEdit"), TEXT("ActivePaletteViewMode"), ActivePaletteViewModeAsInt, GEditorPerProjectIni);
	ActivePaletteViewMode = EFoliagePaletteViewMode::Type(ActivePaletteViewModeAsInt);

	GConfig->GetFloat(TEXT("FoliageEdit"), TEXT("PaletteThumbnailScale"), PaletteThumbnailScale, GEditorPerProjectIni);

	GConfig->GetBool(TEXT("FoliageEdit"), TEXT("IsInSingleInstantiationMode"), IsInSingleInstantiationMode, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("FoliageEdit"), TEXT("IsInSpawnInCurrentLevelMode"), IsInSpawnInCurrentLevelMode, GEditorPerProjectIni);

	int32 SingleInstantiationPlacementModeAsInt = 0;
	GConfig->GetInt(TEXT("FoliageEdit"), TEXT("SingleInstantiationPlacementMode"), SingleInstantiationPlacementModeAsInt, GEditorPerProjectIni);
	SingleInstantiationPlacementMode = EFoliageSingleInstantiationPlacementMode::Type(SingleInstantiationPlacementModeAsInt);
}

/** Save UI settings to ini file */
void FFoliageUISettings::Save()
{
	FString WindowPositionString = FString::Printf(TEXT("%d,%d,%d,%d"), WindowX, WindowY, WindowWidth, WindowHeight);
	GConfig->SetString(TEXT("FoliageEdit"), TEXT("WindowPosition"), *WindowPositionString, GEditorPerProjectIni);

	GConfig->SetFloat(TEXT("FoliageEdit"), TEXT("Radius"), Radius, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("FoliageEdit"), TEXT("PaintDensity"), PaintDensity, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("FoliageEdit"), TEXT("UnpaintDensity"), UnpaintDensity, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("FoliageEdit"), TEXT("bFilterLandscape"), bFilterLandscape, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("FoliageEdit"), TEXT("bFilterStaticMesh"), bFilterStaticMesh, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("FoliageEdit"), TEXT("bFilterBSP"), bFilterBSP, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("FoliageEdit"), TEXT("bFilterFoliage"), bFilterFoliage, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("FoliageEdit"), TEXT("bFilterTranslucent"), bFilterTranslucent, GEditorPerProjectIni);

	GConfig->SetBool(TEXT("FoliageEdit"), TEXT("bShowPaletteItemDetails"), bShowPaletteItemDetails, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("FoliageEdit"), TEXT("bShowPaletteItemTooltips"), bShowPaletteItemTooltips, GEditorPerProjectIni);
	GConfig->SetInt(TEXT("FoliageEdit"), TEXT("ActivePaletteViewMode"), ActivePaletteViewMode, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("FoliageEdit"), TEXT("PaletteThumbnailScale"), PaletteThumbnailScale, GEditorPerProjectIni);

	GConfig->SetBool(TEXT("FoliageEdit"), TEXT("IsInSingleInstantiationMode"), IsInSingleInstantiationMode, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("FoliageEdit"), TEXT("IsInSpawnInCurrentLevelMode"), IsInSpawnInCurrentLevelMode, GEditorPerProjectIni);
	GConfig->SetInt(TEXT("FoliageEdit"), TEXT("SingleInstantiationPlacementMode"), (int32)SingleInstantiationPlacementMode, GEditorPerProjectIni);
}

#undef LOCTEXT_NAMESPACE