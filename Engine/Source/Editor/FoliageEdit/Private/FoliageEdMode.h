// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "EdMode.h"
#include "Engine/EngineBaseTypes.h"
#include "HAL/PlatformCrt.h"
#include "InputCoreTypes.h"
#include "InstancedFoliage.h"
#include "Internationalization/Text.h"
#include "Math/Axis.h"
#include "Math/Box.h"
#include "Math/Color.h"
#include "Math/Rotator.h"
#include "Math/Sphere.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UnrealWidgetFwd.h"
#include "Widgets/Views/SHeaderRow.h"

class AActor;
class AInstancedFoliageActor;
class FCanvas;
class FEditorViewportClient;
class FName;
class FPrimitiveDrawInterface;
class FReferenceCollector;
class FSceneView;
class FUICommandList;
class FViewport;
class HHitProxy;
class UClass;
class UFoliageType;
class ULandscapeComponent;
class ULevel;
class UObject;
class UPrimitiveComponent;
class UStaticMeshComponent;
class UWorld;
struct FAssetData;
struct FHitResult;
struct FViewportClick;
template <typename FuncType> class TFunctionRef;

/** View modes supported by the foliage palette */
namespace EFoliagePaletteViewMode
{
	enum Type
	{
		Thumbnail,
		Tree
	};
}

/** Single instance mode */
namespace EFoliageSingleInstantiationPlacementMode
{
	enum class Type
	{
		All,
		CycleThrough,
		ModeCount
	};
};

const float SingleInstanceModeBrushSize = 20.0f;

// Current user settings in Foliage UI
struct FFoliageUISettings
{
	void Load();
	void Save();

	// Window
	void SetWindowSizePos(int32 NewX, int32 NewY, int32 NewWidth, int32 NewHeight) { WindowX = NewX; WindowY = NewY; WindowWidth = NewWidth; WindowHeight = NewHeight; }
	void GetWindowSizePos(int32& OutX, int32& OutY, int32& OutWidth, int32& OutHeight) { OutX = WindowX; OutY = WindowY; OutWidth = WindowWidth; OutHeight = WindowHeight; }

	// tool
	bool GetPaintToolSelected() const { return bPaintToolSelected ? true : false; }
	void SetPaintToolSelected(bool InbPaintToolSelected) { bPaintToolSelected = InbPaintToolSelected; }
	bool GetReapplyToolSelected() const { return bReapplyToolSelected ? true : false; }
	void SetReapplyToolSelected(bool InbReapplyToolSelected) { bReapplyToolSelected = InbReapplyToolSelected; }
	bool GetSelectToolSelected() const { return bSelectToolSelected ? true : false; }
	void SetSelectToolSelected(bool InbSelectToolSelected) { bSelectToolSelected = InbSelectToolSelected; }
	bool GetLassoSelectToolSelected() const { return bLassoSelectToolSelected ? true : false; }
	void SetLassoSelectToolSelected(bool InbLassoSelectToolSelected) { bLassoSelectToolSelected = InbLassoSelectToolSelected; }
	bool GetPaintBucketToolSelected() const { return bPaintBucketToolSelected ? true : false; }
	void SetPaintBucketToolSelected(bool InbPaintBucketToolSelected) { bPaintBucketToolSelected = InbPaintBucketToolSelected; }
	bool GetReapplyPaintBucketToolSelected() const { return bReapplyPaintBucketToolSelected ? true : false; }
	void SetReapplyPaintBucketToolSelected(bool InbReapplyPaintBucketToolSelected) { bReapplyPaintBucketToolSelected = InbReapplyPaintBucketToolSelected; }
	bool GetEraseToolSelected() const { return bEraseToolSelected ? true : false; }
	void SetEraseToolSelected(bool InbEraseToolSelected) { bEraseToolSelected = InbEraseToolSelected; }

	float GetRadius() const { return (IsInAnySingleInstantiationMode()) ? SingleInstanceModeBrushSize : Radius; }
	void SetRadius(float InRadius) { if (!IsInAnySingleInstantiationMode()) Radius = InRadius; }
	float GetPaintDensity() const { return PaintDensity; }
	void SetPaintDensity(float InPaintDensity) { PaintDensity = InPaintDensity; }
	float GetUnpaintDensity() const { return UnpaintDensity; }
	void SetUnpaintDensity(float InUnpaintDensity) { UnpaintDensity = InUnpaintDensity; }
	bool GetFilterLandscape() const { return bFilterLandscape ? true : false; }
	void SetFilterLandscape(bool InbFilterLandscape) { bFilterLandscape = InbFilterLandscape; }
	bool GetFilterStaticMesh() const { return bFilterStaticMesh ? true : false; }
	void SetFilterStaticMesh(bool InbFilterStaticMesh) { bFilterStaticMesh = InbFilterStaticMesh; }
	bool GetFilterBSP() const { return bFilterBSP ? true : false; }
	void SetFilterBSP(bool InbFilterBSP) { bFilterBSP = InbFilterBSP; }
	bool GetFilterFoliage() const { return bFilterFoliage; }
	void SetFilterFoliage(bool InbFilterFoliage) { bFilterFoliage = InbFilterFoliage; }
	bool GetFilterTranslucent() const { return bFilterTranslucent; }
	void SetFilterTranslucent(bool InbFilterTranslucent) { bFilterTranslucent = InbFilterTranslucent; }

	bool IsInAnySingleInstantiationMode() const { return GetIsInSingleInstantiationMode() || GetIsInQuickSingleInstantiationMode(); }

	bool GetIsInSingleInstantiationMode() const { return IsInSingleInstantiationMode; }
	void SetIsInSingleInstantiationMode(bool InIsInSingleInstantiationMode) { IsInSingleInstantiationMode = InIsInSingleInstantiationMode; }

	bool GetIsInQuickSingleInstantiationMode() const { return IsInQuickSingleInstantiationMode; }
	void SetIsInQuickSingleInstantiationMode(bool InIsInQuickSingleInstantiationMode) { IsInQuickSingleInstantiationMode = InIsInQuickSingleInstantiationMode; }

	bool IsInAnyEraseMode() const { return GetEraseToolSelected() || GetIsInQuickEraseMode(); }
	bool GetIsInQuickEraseMode() const { return IsInQuickEraseMode; }
	void SetIsInQuickEraseMode(bool InIsInQuickEraseMode) { IsInQuickEraseMode = InIsInQuickEraseMode; }

	EFoliageSingleInstantiationPlacementMode::Type GetSingleInstantiationPlacementMode() const { return SingleInstantiationPlacementMode; }
	void SetSingleInstantiationPlacementMode(EFoliageSingleInstantiationPlacementMode::Type InSingleInstantiationPlacementMode) { SingleInstantiationPlacementMode = InSingleInstantiationPlacementMode; }

	int32 GetSingleInstantiationCycleThroughIndex() const { return SingleInstantiationCycleThroughIndex; }
	void IncrementSingleInstantiationCycleThroughIndex() { SingleInstantiationCycleThroughIndex++; }

	bool GetIsInSpawnInCurrentLevelMode() const { return IsInSpawnInCurrentLevelMode; }
	void SetSpawnInCurrentLevelMode(bool InSpawnInCurrentLevelMode) { IsInSpawnInCurrentLevelMode = InSpawnInCurrentLevelMode; }

	bool GetShowPaletteItemDetails() const { return bShowPaletteItemDetails; }
	void SetShowPaletteItemDetails(bool InbShowPaletteItemDetails) { bShowPaletteItemDetails = InbShowPaletteItemDetails; }
	bool GetShowPaletteItemTooltips() const { return bShowPaletteItemTooltips; }
	void SetShowPaletteItemTooltips(bool InbShowPaletteItemTooltips) { bShowPaletteItemTooltips = InbShowPaletteItemTooltips; }
	EFoliagePaletteViewMode::Type GetActivePaletteViewMode() const { return ActivePaletteViewMode; }
	void SetActivePaletteViewMode(EFoliagePaletteViewMode::Type InActivePaletteViewMode) { ActivePaletteViewMode = InActivePaletteViewMode; }
	float GetPaletteThumbnailScale() const { return PaletteThumbnailScale; }
	void SetPaletteThumbnailScale(float InThumbnailScale) { PaletteThumbnailScale = InThumbnailScale; }

	FFoliageUISettings()
		: WindowX(-1)
		, WindowY(-1)
		, WindowWidth(284)
		, WindowHeight(400)
		, bPaintToolSelected(true)
		, bReapplyToolSelected(false)
		, bSelectToolSelected(false)
		, bLassoSelectToolSelected(false)
		, bPaintBucketToolSelected(false)
		, bReapplyPaintBucketToolSelected(false)
		, bEraseToolSelected(false)
		, bShowPaletteItemDetails(true)
		, bShowPaletteItemTooltips(true)
		, ActivePaletteViewMode(EFoliagePaletteViewMode::Thumbnail)
		, PaletteThumbnailScale(0.3f)
		, Radius(250.f)
		, PaintDensity(0.5f)
		, UnpaintDensity(0.f)
		, IsInSingleInstantiationMode(false)
		, IsInQuickSingleInstantiationMode(false)
		, IsInQuickEraseMode(false)
		, SingleInstantiationPlacementMode(EFoliageSingleInstantiationPlacementMode::Type::All)
		, SingleInstantiationCycleThroughIndex(0)
		, IsInSpawnInCurrentLevelMode(false)
		, bFilterLandscape(true)
		, bFilterStaticMesh(true)
		, bFilterBSP(true)
		, bFilterFoliage(false)
		, bFilterTranslucent(false)
	{
	}

	~FFoliageUISettings() = default;

private:
	int32 WindowX;
	int32 WindowY;
	int32 WindowWidth;
	int32 WindowHeight;

	bool bPaintToolSelected;
	bool bReapplyToolSelected;
	bool bSelectToolSelected;
	bool bLassoSelectToolSelected;
	bool bPaintBucketToolSelected;
	bool bReapplyPaintBucketToolSelected;
	bool bEraseToolSelected;

	bool bShowPaletteItemDetails;
	bool bShowPaletteItemTooltips;
	EFoliagePaletteViewMode::Type ActivePaletteViewMode;
	float PaletteThumbnailScale;

	float Radius;
	float PaintDensity;
	float UnpaintDensity;

	bool IsInSingleInstantiationMode;
	bool IsInQuickSingleInstantiationMode;
	bool IsInQuickEraseMode;
	EFoliageSingleInstantiationPlacementMode::Type SingleInstantiationPlacementMode;
	int32 SingleInstantiationCycleThroughIndex;
	bool IsInSpawnInCurrentLevelMode;

public:
	bool bFilterLandscape;
	bool bFilterStaticMesh;
	bool bFilterBSP;
	bool bFilterFoliage;
	bool bFilterTranslucent;
};


struct FFoliageMeshUIInfo
{
	TObjectPtr<UFoliageType>	Settings;
	int32			InstanceCountCurrentLevel;
	int32			InstanceCountTotal;

	FFoliageMeshUIInfo(UFoliageType* InSettings);

	bool operator == (const FFoliageMeshUIInfo& Other) const
	{
		return Settings == Other.Settings;
	}

	FText GetNameText() const;
};

typedef TSharedPtr<FFoliageMeshUIInfo> FFoliageMeshUIInfoPtr;

// Snapshot of current MeshInfo state. Created at start of a brush stroke to store the existing instance info.
class FMeshInfoSnapshot
{
	FFoliageInstanceHash Hash;
	TArray<FVector> Locations;
public:
	FMeshInfoSnapshot(FFoliageInfo* MeshInfo)
		: Hash(*MeshInfo->InstanceHash)
	{
		int32 NumInstances = MeshInfo->Instances.Num();
		Locations.Reserve(NumInstances);
		Locations.AddUninitialized(NumInstances);
		for (int32 Idx = 0; Idx < NumInstances; Idx++)
		{
			Locations[Idx] = MeshInfo->Instances[Idx].Location;
		}
	}

	int32 CountInstancesInsideSphere(const FSphere& Sphere) const
	{
		int32 Count = 0;

		auto TempInstances = Hash.GetInstancesOverlappingBox(FBox::BuildAABB(Sphere.Center, FVector(Sphere.W)));
		for (int32 Idx : TempInstances)
		{
			if (FSphere(Locations[Idx], 0.f).IsInside(Sphere))
			{
				Count++;
			}
		}

		return Count;
	}
};

//
// Painting filtering options
//
struct FFoliagePaintingGeometryFilter
{
	bool bAllowLandscape;
	bool bAllowStaticMesh;
	bool bAllowBSP;
	bool bAllowFoliage;
	bool bAllowTranslucent;

	FFoliagePaintingGeometryFilter(const FFoliageUISettings& InUISettings)
		: bAllowLandscape(InUISettings.bFilterLandscape)
		, bAllowStaticMesh(InUISettings.bFilterStaticMesh)
		, bAllowBSP(InUISettings.bFilterBSP)
		, bAllowFoliage(InUISettings.bFilterFoliage)
		, bAllowTranslucent(InUISettings.bFilterTranslucent)
	{
	}

	FFoliagePaintingGeometryFilter()
		: bAllowLandscape(false)
		, bAllowStaticMesh(false)
		, bAllowBSP(false)
		, bAllowFoliage(false)
		, bAllowTranslucent(false)
	{
	}

	bool operator() (const UPrimitiveComponent* Component) const;
};


// Number of buckets for layer weight histogram distribution.
#define NUM_INSTANCE_BUCKETS 10

enum class EFoliageEditingState : uint8
{
	Unknown,
	Enabled,
	PIEWorld,
	SIEWorld,
};
/**
* Foliage editor mode
*/
class FEdModeFoliage : public FEdMode
{
public:
	FFoliageUISettings UISettings;

	/** Command list lives here so that the key bindings on the commands can be processed in the viewport. */
	TSharedPtr<FUICommandList> UICommandList;

	/** Constructor */
	FEdModeFoliage();

	/** Destructor */
	virtual ~FEdModeFoliage();

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	/** FEdMode: Called when the mode is entered */
	virtual void Enter() override;

	/** FEdMode: Called when the mode is exited */
	virtual void Exit() override;

	/** FEdMode: Called after an Undo operation */
	virtual void PostUndo() override;

	virtual bool UsesToolkits() const override { return true; }

	/** Called when the current level changes */
	void NotifyNewCurrentLevel();
	void NotifyLevelAddedToWorld(ULevel* InLevel, UWorld* InWorld);
	void NotifyLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld);

	/** Called when asset is removed */
	void NotifyAssetRemoved(const FAssetData& AssetInfo);

	/** Called when actor foliage selection changes */
	void NotifyActorSelectionChanged(bool bSelect, const TArray<AActor*>& Selection);

	/**
	* Called when the mouse is moved over the viewport
	*
	* @param	InViewportClient	Level editor viewport client that captured the mouse input
	* @param	InViewport			Viewport that captured the mouse input
	* @param	InMouseX			New mouse cursor X coordinate
	* @param	InMouseY			New mouse cursor Y coordinate
	*
	* @return	true if input was handled
	*/
	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override;

	/**
	* FEdMode: Called when the mouse is moved while a window input capture is in effect
	*
	* @param	InViewportClient	Level editor viewport client that captured the mouse input
	* @param	InViewport			Viewport that captured the mouse input
	* @param	InMouseX			New mouse cursor X coordinate
	* @param	InMouseY			New mouse cursor Y coordinate
	*
	* @return	true if input was handled
	*/
	virtual bool CapturedMouseMove(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InMouseX, int32 InMouseY) override;

	/** FEdMode: Called when a mouse button is pressed */
	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;

	/** FEdMode: Called when a mouse button is released */
	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;

	/** FEdMode: Called once per frame */
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;

	/** FEdMode: Called when a key is pressed */
	virtual bool InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) override;

	/** FEdMode: Called when mouse drag input it applied */
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;

	/** FEdMode: Render elements for the Foliage tool */
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;

	/** FEdMode: Render HUD elements for this tool */
	virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;

	/** FEdMode: Handling SelectActor */
	virtual bool Select(AActor* InActor, bool bInSelected) override;

	/** FEdMode: Check to see if an actor can be selected in this mode - no side effects */
	virtual bool IsSelectionAllowed(AActor* InActor, bool bInSelection) const override;

	/** FEdMode: Called when the currently selected actor has changed */
	virtual void ActorSelectionChangeNotify() override;

	/** Notifies all active modes of mouse click messages. */
	bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy *HitProxy, const FViewportClick &Click) override;

	/** Moves selected foliage instances to the target level. */
	void MoveSelectedFoliageToLevel(ULevel* InTargetLevel);

	/** Tell us if we can moves selected foliage instances to the target level. */
	bool CanMoveSelectedFoliageToLevel(ULevel* InTargetLevel) const;
	
	/** Moves selected foliage instances to the current actor editor context */
	void MoveSelectedFoliageToActorEditorContext();

	/** Ends tracking and end potential transaction */
	bool EndTracking();

	/** FEdMode: widget handling */
	virtual FVector GetWidgetLocation() const override;
	virtual bool AllowWidgetMove() override;
	virtual bool ShouldDrawWidget() const override;
	virtual bool UsesTransformWidget() const override;
	virtual EAxisList::Type GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const override;

	virtual bool DisallowMouseDeltaTracking() const override;

	/** Called when objects are replaced (after a BP compile for instance) */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

	/** Forces real-time perspective viewports */
	void ForceRealTimeViewports(const bool bEnable);

	/** Start foliage tracing */
	void StartFoliageBrushTrace(FEditorViewportClient* ViewportClient, class UViewportInteractor* Interactor = nullptr);

	/* End foliage tracing*/
	void EndFoliageBrushTrace();

	/** Trace under the mouse cursor and update brush position */
	void FoliageBrushTrace(FEditorViewportClient* ViewportClient, const FVector& InRayOrigin, const FVector& InRayDirection);

	/** Generate start/end points for a random trace inside the sphere brush.
	returns a line segment inside the sphere parallel to the view direction */
	void GetRandomVectorInBrush(FVector& OutStart, FVector& OutEnd);

	/** Setup before call to ApplyBrush */
	void PreApplyBrush();

	/** Apply brush */
	void ApplyBrush(FEditorViewportClient* ViewportClient);

	/** Get list of meshes for current level */
	TArray<FFoliageMeshUIInfoPtr>& GetFoliageMeshList();

	/** Populate mesh with foliage mesh settings found across world */
	void PopulateFoliageMeshList();

	/** Handler for mesh list sort mode changes */
	void OnFoliageMeshListSortModeChanged(EColumnSortMode::Type InSortMode);

	/** Returns foliage mesh list sort mode */
	EColumnSortMode::Type GetFoliageMeshListSortMode() const;

	/** Handler for foliage mesh instance count changes */
	void OnInstanceCountUpdated(const UFoliageType* FoliageType);

	/** Counts total number of instances in current level and across whole world */
	void CalcTotalInstanceCount(int32& OutInstanceCountTotal, int32& OutInstanceCountCurrentLevel);

	/** Whether any of the selected foliage types can be painted into level */
	bool CanPaint(const ULevel* InLevel);

	/** Whether specified FoliageType can be painted into level */
	static bool CanPaint(const UFoliageType* FoliageType, const ULevel* InLevel);

	/** Shift or modifier button pressed */
	bool IsModifierButtonPressed(const FEditorViewportClient* ViewportClient) const;

	/** Add a new asset (FoliageType or StaticMesh) */
	UFoliageType* AddFoliageAsset(UObject* InAsset, bool bInPlaceholderAsset = false);

	/** Remove a list of Foliage types */
	bool RemoveFoliageType(UFoliageType** FoliageTypes, int32 Num);

	/** Reapply cluster settings to all the instances */
	void ReallocateClusters(UFoliageType* Settings);

	/** Bake meshes to StaticMeshActors */
	void BakeFoliage(UFoliageType* Settings, bool bSelectedOnly);

	/**
	* Copy the settings object for this static mesh
	*
	* @param StaticMesh	The static mesh to copy the settings of.
	*
	* @return The duplicated settings now assigned to the static mesh.
	*/
	UFoliageType* CopySettingsObject(UFoliageType* Settings);

	/** Replace the settings object for this static mesh with the one specified */
	void ReplaceSettingsObject(UFoliageType* OldSettings, UFoliageType* NewSettings);

	/** Save the foliage type object. If it isn't an asset, will prompt the user for a location to save the new asset. */
	UFoliageType* SaveFoliageTypeObject(UFoliageType* Settings, bool bPlaceholderAsset = false);

	void IncludeNonFoliageActors(const TArray<const UFoliageType*>& FoliageTypes, bool bOnlyCurrentLevel);

	void ExcludeFoliageActors(const TArray<const UFoliageType*>& FoliageTypes, bool bOnlyCurrentLevel);

	/** Set/Clear selection for foliage instances of a specific types */
	void SelectInstances(const TArray<const UFoliageType*>& FoliageTypes, bool bSelect);

	/** Set/Clear selection for foliage instances of specific type  */
	void SelectInstances(const UFoliageType* Settings, bool bSelect);

	/*Focus on selected instances*/
	void FocusSelectedInstances() const;

	/** Find and select instances that don't have valid base or 'off-ground' */
	void SelectInvalidInstances(const TArray<const UFoliageType*>& FoliageTypes);

	/** Find and select instances that don't have valid base or 'off-ground' */
	void SelectInvalidInstances(const UFoliageType* Settings);

	/** Returns selected foliage types based on selected foliage instances */
	void GetSelectedInstanceFoliageTypes(TArray<const UFoliageType*>& OutFoliageTypes) const;

	/** Adjusts the radius of the foliage brush, using the given multiplier to adjust speed */
	void AdjustBrushRadius(float Multiplier);

	/** Adjusts the painting density of the foliage brush, using the given multiplier to adjust speed */
	void AdjustPaintDensity(float Multiplier);

	/** Adjusts the unpainting (erasing) density of the foliage brush, using the given multiplier to adjust speed */
	void AdjustUnpaintDensity(float Multiplier);

	/** Add desired instances. Uses foliage settings to determine location/scale/rotation and whether instances should be ignored */
	static void AddInstances(UWorld* InWorld, const TArray<FDesiredFoliageInstance>& DesiredInstances, const FFoliagePaintingGeometryFilter& OverrideGeometryFilter, bool InRebuildFoliageTree = true);

	/** Called as PIE ends */
	void OnEndPIE(const bool bIsSimulating);

	/** Return the current foliage editing state */
	EFoliageEditingState GetEditingState() const;

	/** Simgple wrapper to know if we can edit foliage based on edit state */
	bool IsEditingEnabled() const
	{
		return GetEditingState() == EFoliageEditingState::Enabled;
	}

	typedef TMap<FName, TMap<ULandscapeComponent*, TArray<uint8> > > LandscapeLayerCacheData;

	FSimpleMulticastDelegate OnToolChanged;

	/** Sets the tool mode to Paint. */
	void OnSetPaint();

	/** Sets the tool mode to Reapply Settings. */
	void OnSetReapplySettings();

	/** Sets the tool mode to Select. */
	void OnSetSelectInstance();

	/** Sets the tool mode to Lasso Select. */
	void OnSetLasso();

	/** Sets the tool mode to Paint Bucket. */
	void OnSetPaintFill();

	/** Sets the tool mode to Erase */
	void OnSetErase();

	/** Sets the tool mode to Place Single Instance*/
	void OnSetPlace();

	/** Handle reflecting selected foliage types in the FoliagePalette */
	void OnReflectSelectionInPalette();

	/** Remove currently selected instances*/
	void RemoveSelectedInstances(UWorld* InWorld);

	/** Returns the list of valid FoliageType class filters */
	void GetFoliageTypeFilters(TArray<const UClass*>& OutFilters) const;

	/*Find the relevant foliage actor with the foliage type and run the operation*/
	static void ForEachFoliageInfo(UWorld* InWorld, const UFoliageType* FoliageType, const FSphere& BrushSphere, TFunctionRef<bool(AInstancedFoliageActor* IFA, FFoliageInfo* FoliageInfo, const UFoliageType* FoliageType)> InOperation);
	
private:

	void BindCommands();
	bool CurrentToolUsesBrush() const;

	/** Called when the user changes the current tool in the UI */
	void HandleToolChanged();

	/** Deselects all tools */
	void ClearAllToolSelection();

	/** Add instances inside the brush to match DesiredInstanceCount */
	void AddInstancesForBrush(UWorld* InWorld, const UFoliageType* Settings, const FSphere& BrushSphere, int32 DesiredInstanceCount, float Pressure);

	/** Add single instance inside the brush 
	* @return true if instance was added successfully
	*/
	bool AddSingleInstanceForBrush(UWorld* InWorld, const UFoliageType* Settings, float Pressure);

	/** Remove instances inside the brush to match DesiredInstanceCount. */
	void RemoveInstancesForBrush(UWorld* InWorld, const UFoliageType* Settings, const FSphere& BrushSphere, int32 DesiredInstanceCount, float Pressure);

	/** Apply paint bucket to actor */
	void ApplyPaintBucket_Add(AActor* Actor);
	void ApplyPaintBucket_Remove(AActor* Actor);

	/** Reapply instance settings to exiting instances */
	void ReapplyInstancesDensityForBrush(UWorld* InWorld, const UFoliageType* Settings, const FSphere& BrushSphere, float Pressure);
	void ReapplyInstancesForBrush(UWorld* InWorld, const UFoliageType* Settings, const FSphere& BrushSphere, float Pressure, bool bSingleInstanceMode);
	void ReapplyInstancesForBrush(UWorld* InWorld, AInstancedFoliageActor* IFA, const UFoliageType* Settings, FFoliageInfo* MeshInfo, const FSphere& BrushSphere, float Pressure, bool bSingleInstanceMode);

	/** Select instances inside the brush. */
	void SelectInstancesForBrush(UWorld* InWorld, const UFoliageType* Settings, const FSphere& BrushSphere, bool bSelect);

	/** Select instance closest to the brush. */
	void SelectInstanceAtLocation(UWorld* InWorld, const UFoliageType* Settings, const FVector& BrushLocation, bool bSelect);

	/** Set/Clear selection for all foliage instances */
	void SelectInstances(UWorld* InWorld, bool bSelect);

	/** Set/Clear selection for foliage instances of specific type  */
	void SelectInstances(UWorld* InWorld, const UFoliageType* Settings, bool bSelect);

	/**  Propagate the selected foliage instances to the actual render foliage */
	void ApplySelection(UWorld* InWorld, bool bApply);

	/**  Update Instances so that they are in the right partitioning level */
	void UpdateInstancePartitioning(UWorld* InWorld);

	/** Called when transform transaction is done */
	void PostTransformSelectedInstances(UWorld* InWorld);

	/**  Applies relative transformation to selected instances */
	void TransformSelectedInstances(UWorld* InWorld, const FVector& InDrag, const FRotator& InRot, const FVector& InScale, bool bDuplicate);

	/**  Return true if OutLocation is valid */
	bool GetSelectionLocation(UWorld* InWorld, FVector& OutLocation) const;
	
	/**  Updates ed mode widget location to currently selected instance */
	void UpdateWidgetLocationToInstanceSelection();

	/** Snap instance to the ground   */
	bool SnapInstanceToGround(AInstancedFoliageActor* InIFA, const UFoliageType* Settings, FFoliageInfo& Mesh, int32 InstanceIdx);
	void SnapSelectedInstancesToGround(UWorld* InWorld);

	/** Callback for when an actor is spawned (to check if it's a new IFA) */
	void HandleOnActorSpawned(AActor* Actor);

	/** Callback for when the mesh assigned to a foliage type referenced by an IFA is changed */
	void HandleOnFoliageTypeMeshChanged(UFoliageType* FoliageType);

	/** Common code for adding instances to world based on settings */
	static bool AddInstancesImp(UWorld* InWorld, const UFoliageType* Settings, const TArray<FDesiredFoliageInstance>& DesiredInstances, const TArray<int32>& ExistingInstances = TArray<int32>(), const float Pressure = 1.f, LandscapeLayerCacheData* LandscapeLayerCaches = nullptr, const FFoliageUISettings* UISettings = nullptr, const FFoliagePaintingGeometryFilter* OverrideGeometryFilter = nullptr, bool InRebuildFoliageTree = true);

	/** Logic for determining which instances can be placed in the world*/
	static void CalculatePotentialInstances(UWorld* InWorld, const UFoliageType* Settings, const TArray<FDesiredFoliageInstance>& DesiredInstances, TArray<FPotentialInstance> OutPotentialInstances[NUM_INSTANCE_BUCKETS], LandscapeLayerCacheData* LandscaleLayerCachesPtr, const FFoliageUISettings* UISettings, const FFoliagePaintingGeometryFilter* OverrideGeometryFilter = nullptr);

	/** Similar to CalculatePotentialInstances, but it doesn't do any overlap checks which are much harder to thread. Meant to be run in parallel for placing lots of instances */
	static void CalculatePotentialInstances_ThreadSafe(UWorld* InWorld, const UFoliageType* Settings, const TArray<FDesiredFoliageInstance>* DesiredInstances, TArray<FPotentialInstance> OutPotentialInstances[NUM_INSTANCE_BUCKETS], const FFoliageUISettings* UISettings, const int32 StartIdx, const int32 LastIdx, const FFoliagePaintingGeometryFilter* OverrideGeometryFilter = nullptr);

	/** Lookup the vertex color corresponding to a location traced on a static mesh */
	static bool GetStaticMeshVertexColorForHit(const UStaticMeshComponent* InStaticMeshComponent, int32 InTriangleIndex, const FVector& InHitLocation, FColor& OutVertexColor);

	/** Returns true when at least one color channel is used by the vertex color mask */
	static bool IsUsingVertexColorMask(const UFoliageType* Settings);

	/** Does a filter based on the vertex color of a static mesh */
	static bool VertexMaskCheck(const FHitResult& Hit, const UFoliageType* Settings);

	/** Set the brush mesh opacity */
	void SetBrushOpacity(const float InOpacity);

	float GetPaintingBrushRadius() const;

	/** Called if the foliage tree is outdated */
	void RebuildFoliageTree(const UFoliageType* Settings);

	/** Increments a counter that prevents sending out notifications until end selection is called  and counter reaches 0 */
	void BeginSelectionUpdate();

	/** Decrements counter and will notify editor of selection change if counter reaches 0 */
	void EndSelectionUpdate();

	bool bBrushTraceValid;
	FVector BrushLocation;
	FVector BrushNormal;
	FVector BrushTraceDirection;
	TObjectPtr<UStaticMeshComponent> SphereBrushComponent;

	/** The dynamic material of the sphere brush. */
	class UMaterialInstanceDynamic* BrushMID;
	FColor BrushDefaultHighlightColor;
	FColor BrushCurrentHighlightColor;

	/** Default opacity received from the brush material to reset it when closing. */
	float DefaultBrushOpacity;

	// Landscape layer cache data
	LandscapeLayerCacheData LandscapeLayerCaches;

	// Cache of instance positions at the start of the transaction
	TMultiMap<UFoliageType*, FMeshInfoSnapshot> InstanceSnapshot;

	bool bToolActive;
	bool bCanAltDrag;

	TArray<FFoliageMeshUIInfoPtr>	FoliageMeshList;
	EColumnSortMode::Type			FoliageMeshListSortMode;

	FDelegateHandle OnActorSpawnedHandle;

	int32 UpdateSelectionCounter;
	bool bHasDeferredSelectionNotification;
	friend class FEdModeFoliageSelectionUpdate;

	/** When transforming instances */
	bool bMoving;

	/** Flag to know when we are tracking a transaction in mouse delta */
	bool bTracking;
};

