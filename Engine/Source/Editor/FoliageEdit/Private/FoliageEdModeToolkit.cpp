// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageEdModeToolkit.h"

#include "Editor.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "SFoliageEdit.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SWidget;

#define LOCTEXT_NAMESPACE "FoliageEditMode"

namespace 
{
	static const FName FoliageName(TEXT("Foliage")); 
	const TArray<FName> FoliagePaletteNames = { FoliageName };
}

void FFoliageEdModeToolkit::Init(const TSharedPtr< class IToolkitHost >& InitToolkitHost)
{
	FoliageEdWidget = SNew(SFoliageEdit);

	FModeToolkit::Init(InitToolkitHost);
}

FName FFoliageEdModeToolkit::GetToolkitFName() const
{
	return FName("FoliageEditMode");
}

FText FFoliageEdModeToolkit::GetBaseToolkitName() const
{
	return LOCTEXT( "ToolkitName", "Foliage" );
}

class FEdMode* FFoliageEdModeToolkit::GetEditorMode() const
{
	return GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Foliage);
}

TSharedPtr<SWidget> FFoliageEdModeToolkit::GetInlineContent() const
{
	return FoliageEdWidget;
}

void FFoliageEdModeToolkit::RefreshFullList()
{
	FoliageEdWidget->RefreshFullList();
}

void FFoliageEdModeToolkit::NotifyFoliageTypeMeshChanged(class UFoliageType* FoliageType)
{
	FoliageEdWidget->NotifyFoliageTypeMeshChanged(FoliageType);
}

void FFoliageEdModeToolkit::ReflectSelectionInPalette()
{
	FoliageEdWidget->ReflectSelectionInPalette();
}

void FFoliageEdModeToolkit::GetToolPaletteNames(TArray<FName>& InPaletteName) const
{
	InPaletteName = FoliagePaletteNames;
}

FText FFoliageEdModeToolkit::GetToolPaletteDisplayName(FName PaletteName) const
{
	if (PaletteName == FoliageName)
	{
		return LOCTEXT("Foliage", "Foliage");
	}
	return FText();
}

void FFoliageEdModeToolkit::BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolBarBuilder)
{
	if (PaletteName == FoliageName)
	{
		FoliageEdWidget->CustomizeToolBarPalette(ToolBarBuilder);
	}
}

void FFoliageEdModeToolkit::OnToolPaletteChanged(FName PaletteName) 
{

}

FText FFoliageEdModeToolkit::GetActiveToolDisplayName() const
{
	return FoliageEdWidget->GetActiveToolName();
}

FText FFoliageEdModeToolkit::GetActiveToolMessage() const
{

	return FoliageEdWidget->GetActiveToolMessage();
}


#undef LOCTEXT_NAMESPACE
