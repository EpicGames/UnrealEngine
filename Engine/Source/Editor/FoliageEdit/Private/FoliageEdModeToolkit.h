// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "EdMode.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/BaseToolkit.h"
#include "UObject/NameTypes.h"

/**
 * Public interface to Foliage Edit mode.
 */
class FFoliageEdModeToolkit : public FModeToolkit
{
public:
	/** Initializes the foliage mode toolkit */
	virtual void Init(const TSharedPtr< class IToolkitHost >& InitToolkitHost) override;

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual class FEdMode* GetEditorMode() const override;
	virtual TSharedPtr<class SWidget> GetInlineContent() const override;

	/** Mode Toolbar Palettes **/
	virtual void GetToolPaletteNames(TArray<FName>& InPaletteName) const;
	virtual FText GetToolPaletteDisplayName(FName PaletteName) const; 
	virtual void BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolbarBuilder);

	/** Modes Panel Header Information **/
	virtual FText GetActiveToolDisplayName() const;
	virtual FText GetActiveToolMessage() const;

	virtual void OnToolPaletteChanged(FName PaletteName) override;

	void RefreshFullList();
	void NotifyFoliageTypeMeshChanged(class UFoliageType* FoliageType);
	void ReflectSelectionInPalette();

private:
	TSharedPtr< class SFoliageEdit > FoliageEdWidget;
};
