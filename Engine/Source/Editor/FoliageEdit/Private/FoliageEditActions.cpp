// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageEditActions.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"



#define LOCTEXT_NAMESPACE "FoliageEditCommands"

void FFoliageEditCommands::RegisterCommands()
{
	UI_COMMAND( DecreaseBrushSize, "Decrease Brush Size", "Decreases the size of the foliage brush", EUserInterfaceActionType::Button, FInputChord(EKeys::LeftBracket) );
	UI_COMMAND( IncreaseBrushSize, "Increase Brush Size", "Increases the size of the foliage brush", EUserInterfaceActionType::Button, FInputChord(EKeys::RightBracket) );

	UI_COMMAND( DecreasePaintDensity, "Decrease Brush Density", "Decreases the density of the foliage brush", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::LeftBracket) );
	UI_COMMAND( IncreasePaintDensity, "Increase Brush Density", "Increases the density of the foliage brush", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::RightBracket) );

	UI_COMMAND( DecreaseUnpaintDensity, "Decrease Erase Density", "Decreases the density of the foliage eraser", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::LeftBracket));
	UI_COMMAND( IncreaseUnpaintDensity, "Increase Erase Density", "Increases the density of the foliage eraser", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::RightBracket));

	UI_COMMAND( SetPaint, "Paint", "Paint", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetReapplySettings, "Reapply", "Reapply settings to instances", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetSelect, "Select", "Select", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetLassoSelect, "Lasso", "Lasso Select", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetPaintBucket, "Fill", "Paint Bucket", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( SetNoSettings, "Hide Details", "Hide details.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetPaintSettings, "Show Painting settings", "Show painting settings.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetClusterSettings, "Show Instance settings", "Show settings for placed instances.", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND(ReflectSelectionInPalette, "Reflect Selection in Palette.", "Select foliage type in palette based on selected foliage instances.", EUserInterfaceActionType::None, FInputChord());
}

#undef LOCTEXT_NAMESPACE
