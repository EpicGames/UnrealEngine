// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

/**
 * Unreal Foliage Edit mode actions
 */
class FFoliageEditCommands : public TCommands<FFoliageEditCommands>
{

public:
	FFoliageEditCommands() : TCommands<FFoliageEditCommands>
	(
		"FoliageEditMode", // Context name for fast lookup
		NSLOCTEXT("Contexts", "FoliageEditMode", "Foliage Edit Mode"), // Localized context name for displaying
		NAME_None, // Parent
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
	{
	}
	
	/**
	 * Foliage Edit Commands
	 */
	
	/** Commands for the foliage brush settings. */
	TSharedPtr< FUICommandInfo > IncreaseBrushSize;
	TSharedPtr< FUICommandInfo > DecreaseBrushSize;

	TSharedPtr< FUICommandInfo > IncreasePaintDensity;
	TSharedPtr< FUICommandInfo > DecreasePaintDensity;

	TSharedPtr< FUICommandInfo > IncreaseUnpaintDensity;
	TSharedPtr< FUICommandInfo > DecreaseUnpaintDensity;

	/** Commands for the tools toolbar. */
	TSharedPtr< FUICommandInfo > SetPaint;
	TSharedPtr< FUICommandInfo > SetReapplySettings;
	TSharedPtr< FUICommandInfo > SetSelect;
	TSharedPtr< FUICommandInfo > SetLassoSelect;
	TSharedPtr< FUICommandInfo > SetPaintBucket;

	/** Commands for the foliage item toolbar. */
	TSharedPtr< FUICommandInfo > SetNoSettings;
	TSharedPtr< FUICommandInfo > SetPaintSettings;
	TSharedPtr< FUICommandInfo > SetClusterSettings;

	/** FoliagePalette commands*/
	TSharedPtr< FUICommandInfo > ReflectSelectionInPalette;

	/** FoliageType commands */
	TSharedPtr< FUICommandInfo > RemoveFoliageType;
	TSharedPtr< FUICommandInfo > ShowFoliageTypeInCB;
	TSharedPtr< FUICommandInfo > SelectAllInstances;
	TSharedPtr< FUICommandInfo > DeselectAllInstances;
	TSharedPtr< FUICommandInfo > SelectInvalidInstances;
	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;

public:
};
