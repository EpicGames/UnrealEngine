// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageEditModule.h"
#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/EditorExperimentalSettings.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "EditorModeRegistry.h"
#include "EditorModes.h"
#include "UnrealEdGlobals.h"
#include "AssetRegistry/AssetRegistryModule.h"

const FName FoliageEditAppIdentifier = FName(TEXT("FoliageEdApp"));

#include "UObject/UObjectIterator.h"
#include "FoliageType_InstancedStaticMesh.h"
#include "FoliageType_Actor.h"
#include "InstancedFoliageActor.h"
#include "FoliageEdMode.h"
#include "FoliageEditActions.h"
#include "PropertyEditorModule.h"
#include "FoliageTypeDetails.h"
#include "ProceduralFoliageComponent.h"
#include "ProceduralFoliageComponentVisualizer.h"
#include "ProceduralFoliageComponentDetails.h"
#include "ActorFactoryProceduralFoliage.h"
#include "ProceduralFoliageVolume.h"
#include "ProceduralFoliageBlockingVolume.h"
#include "FoliageTypeObjectCustomization.h"
#include "FoliageType_ISMThumbnailRenderer.h"
#include "FoliageType_ActorThumbnailRenderer.h"
#include "EditorModeManager.h"
#include "LevelEditorViewport.h"

/**
 * Foliage Edit Mode module
 */
class FFoliageEditModule : public IFoliageEditModule
{
public:

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override
	{
		FEditorModeRegistry::Get().RegisterMode<FEdModeFoliage>(
			FBuiltinEditorModes::EM_Foliage,
			NSLOCTEXT("EditorModes", "FoliageMode", "Foliage"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.FoliageMode", "LevelEditor.FoliageMode.Small"),
			true, 400
			);

		FFoliageEditCommands::Register();

		// Register the details customizer
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout("FoliageType", FOnGetDetailCustomizationInstance::CreateStatic(&FFoliageTypeDetails::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("FoliageTypeObject", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FFoliageTypeObjectCustomization::MakeInstance));
		
		if (GUnrealEd)
		{
			GUnrealEd->RegisterComponentVisualizer(UProceduralFoliageComponent::StaticClass()->GetFName(), MakeShareable(new FProceduralFoliageComponentVisualizer));
		}

		FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditor.RegisterCustomClassLayout("ProceduralFoliageComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FProceduralFoliageComponentDetails::MakeInstance));

		// Actor Factories
		auto ProceduralFoliageVolumeFactory = NewObject<UActorFactoryProceduralFoliage>();
		GEditor->ActorFactories.Add(ProceduralFoliageVolumeFactory);

#if WITH_EDITOR
		// Volume placeability
		if (!GetDefault<UEditorExperimentalSettings>()->bProceduralFoliage)
		{
			AProceduralFoliageVolume::StaticClass()->ClassFlags |= CLASS_NotPlaceable;
			AProceduralFoliageBlockingVolume::StaticClass()->ClassFlags |= CLASS_NotPlaceable;
		}

		SubscribeEvents();
#endif
		
		// Register thumbnail renderer
		UThumbnailManager::Get().RegisterCustomRenderer(UFoliageType_InstancedStaticMesh::StaticClass(), UFoliageType_ISMThumbnailRenderer::StaticClass());
		UThumbnailManager::Get().RegisterCustomRenderer(UFoliageType_Actor::StaticClass(), UFoliageType_ActorThumbnailRenderer::StaticClass());

		IFoliageEditModuleBase::SetFoliageEditModulePtr(this);
	}

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override
	{
		IFoliageEditModuleBase::SetFoliageEditModulePtr(nullptr);

		if (GUnrealEd)
		{
			GUnrealEd->UnregisterComponentVisualizer(UProceduralFoliageComponent::StaticClass()->GetFName());
		}

		FFoliageEditCommands::Unregister();

		FEditorModeRegistry::Get().UnregisterMode(FBuiltinEditorModes::EM_Foliage);

		if (!UObjectInitialized())
		{
			return;
		}

#if WITH_EDITOR
		UnsubscribeEvents();
#endif

		// Unregister the details customization
		if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.UnregisterCustomClassLayout("FoliageType");
			PropertyModule.NotifyCustomizationModuleChanged();
		}
	}

#if WITH_EDITOR

	void OnLevelActorDeleted(AActor* Actor)
	{
		if (AProceduralFoliageVolume* ProceduralFoliageVolume = Cast<AProceduralFoliageVolume>(Actor))
		{
			if (UProceduralFoliageComponent* ProceduralComponent = ProceduralFoliageVolume->ProceduralComponent)
			{
				ProceduralComponent->RemoveProceduralContent();
			}
		}
	}

	void NotifyAssetRemoved(const FAssetData& AssetInfo)
	{
		// Go through all FoliageActors in the world and delete 
		for(TObjectIterator<AInstancedFoliageActor> It; It; ++It)
		{
			AInstancedFoliageActor* IFA = *It;
			IFA->CleanupDeletedFoliageType();
		}
	}

	void SubscribeEvents()
	{
		GEngine->OnLevelActorDeleted().Remove(OnLevelActorDeletedDelegateHandle);
		OnLevelActorDeletedDelegateHandle = GEngine->OnLevelActorDeleted().AddRaw(this, &FFoliageEditModule::OnLevelActorDeleted);

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		AssetRegistryModule.Get().OnAssetRemoved().AddRaw(this, &FFoliageEditModule::NotifyAssetRemoved);

		auto ExperimentalSettings = GetMutableDefault<UEditorExperimentalSettings>();
		ExperimentalSettings->OnSettingChanged().Remove(OnExperimentalSettingChangedDelegateHandle);
		OnExperimentalSettingChangedDelegateHandle =  ExperimentalSettings->OnSettingChanged().AddRaw(this, &FFoliageEditModule::HandleExperimentalSettingChanged);
	}

	void UnsubscribeEvents()
	{
		GEngine->OnLevelActorDeleted().Remove(OnLevelActorDeletedDelegateHandle);
		GetMutableDefault<UEditorExperimentalSettings>()->OnSettingChanged().Remove(OnExperimentalSettingChangedDelegateHandle);

		if (FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry")))
		{
			IAssetRegistry* AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).TryGet();
			if (AssetRegistry)
			{
				AssetRegistry->OnAssetRemoved().RemoveAll(this);
			}
		}
	}

	void HandleExperimentalSettingChanged(FName PropertyName)
	{
		// Update the volume visibility flags
		TArray<UClass*> PreviousVolumeClasses;
		UUnrealEdEngine::GetSortedVolumeClasses(&PreviousVolumeClasses);

		if (GetDefault<UEditorExperimentalSettings>()->bProceduralFoliage)
		{
			AProceduralFoliageVolume::StaticClass()->ClassFlags &= ~CLASS_NotPlaceable;
			AProceduralFoliageBlockingVolume::StaticClass()->ClassFlags &= ~CLASS_NotPlaceable;
		}
		else
		{
			AProceduralFoliageVolume::StaticClass()->ClassFlags |= CLASS_NotPlaceable;
			AProceduralFoliageBlockingVolume::StaticClass()->ClassFlags |= CLASS_NotPlaceable;
		}

		// Update the volume visibility flags
		TArray<UClass*> VolumeClasses;
		UUnrealEdEngine::GetSortedVolumeClasses(&VolumeClasses);

		// Update the visibility state of each actor for each viewport
		for(FLevelEditorViewportClient* ViewClient : GUnrealEd->GetLevelViewportClients())
		{			
			// Backup previous values
			TMap<UClass*, bool> PreviousVolumeClassVisibility;

			for (int32 i = 0; i < ViewClient->VolumeActorVisibility.Num(); ++i)
			{
				PreviousVolumeClassVisibility.Add(PreviousVolumeClasses[i], ViewClient->VolumeActorVisibility[i]);
			}

			// Resize the array to fix with the new values
			ViewClient->VolumeActorVisibility.Init(true, VolumeClasses.Num());

			// Reapply previous values
			for (int32 i = 0; i < ViewClient->VolumeActorVisibility.Num(); ++i)
			{
				const bool* PreviousVisiblity = PreviousVolumeClassVisibility.Find(VolumeClasses[i]);
				ViewClient->VolumeActorVisibility[i] = PreviousVisiblity != nullptr ? *PreviousVisiblity : true;
			}
		}

		GUnrealEd->UpdateVolumeActorVisibility();
	}

	virtual void MoveSelectedFoliageToLevel(ULevel* InTargetLevel) override
	{
		ensure(GLevelEditorModeTools().IsModeActive(FBuiltinEditorModes::EM_Foliage));

		FEdModeFoliage* FoliageMode = (FEdModeFoliage*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Foliage);

		FoliageMode->MoveSelectedFoliageToLevel(InTargetLevel);
	}

	virtual void UpdateMeshList() override
	{
		FEditorModeTools& EditorModeTools = GLevelEditorModeTools(); 
		if (EditorModeTools.IsModeActive(FBuiltinEditorModes::EM_Foliage))
		{
			FEdModeFoliage* FoliageMode = (FEdModeFoliage*)EditorModeTools.GetActiveMode(FBuiltinEditorModes::EM_Foliage);
			FoliageMode->PopulateFoliageMeshList();
		}
	}

	virtual bool CanMoveSelectedFoliageToLevel(ULevel* InTargetLevel) const override
	{
		ensure(GLevelEditorModeTools().IsModeActive(FBuiltinEditorModes::EM_Foliage));

		FEdModeFoliage* FoliageMode = (FEdModeFoliage*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Foliage);

		return FoliageMode->CanMoveSelectedFoliageToLevel(InTargetLevel);
	}

	virtual void RegisterComponentBaseIDClassToIgnore(TSubclassOf<UActorComponent> InClassToIgnore) override
	{
		ComponentBaseIDClassesToIgnore.Add(InClassToIgnore.Get());
	}

	virtual void UnregisterComponentBaseIDClassToIgnore(TSubclassOf<UActorComponent> InClassToUnregister) override
	{
		ComponentBaseIDClassesToIgnore.Remove(InClassToUnregister.Get());
	}

	virtual bool ShouldIgnoreComponentForBaseID(UActorComponent* InComponent) const
	{
		for (TSoftObjectPtr<UClass> ClassToCheckPtr : ComponentBaseIDClassesToIgnore)
		{
			if (UClass* ClassToCheck = ClassToCheckPtr.Get())
			{
				if (InComponent->IsA(ClassToCheck))
				{
					return true;
				}
			}
		}
		return false;
	}

	FDelegateHandle OnLevelActorDeletedDelegateHandle;
	FDelegateHandle OnExperimentalSettingChangedDelegateHandle;

	TSet<TSoftObjectPtr<UClass>> ComponentBaseIDClassesToIgnore;
#endif
};

IMPLEMENT_MODULE( FFoliageEditModule, FoliageEdit );
