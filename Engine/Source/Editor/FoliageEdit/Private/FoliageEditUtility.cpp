// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageEditUtility.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "FileHelpers.h"
#include "FoliageType.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "InstancedFoliage.h"
#include "InstancedFoliageActor.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "LevelUtils.h"
#include "Misc/AssertionMacros.h"
#include "Misc/PackageName.h"
#include "ScopedTransaction.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/UniqueObj.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#define LOCTEXT_NAMESPACE "FoliageEdMode"

UFoliageType* FFoliageEditUtility::SaveFoliageTypeObject(UFoliageType* InFoliageType, bool bInPlaceholderAsset)
{
	UFoliageType* TypeToSave = nullptr;

	if (!InFoliageType->IsAsset())
	{
		FString PackageName;
		FString AssetName = InFoliageType->GetDefaultNewAssetName();
		UObject* FoliageSource = InFoliageType->GetSource();
		if (FoliageSource)
		{
			// Avoid using source name if this is a placeholder asset which is going to be replaced 
			if (!bInPlaceholderAsset)
			{
				AssetName = FoliageSource->GetName() + TEXT("_FoliageType");
			}

			// Build default settings asset name and path
			PackageName = FPackageName::GetLongPackagePath(FoliageSource->GetOutermost()->GetName()) + TEXT("/") + AssetName;
		}

		FSaveAssetDialogConfig SaveAssetDialogConfig;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
		SaveAssetDialogConfig.DefaultPath = FPaths::GetPath(PackageName);
		SaveAssetDialogConfig.DefaultAssetName = AssetName;
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
		if(!SaveObjectPath.IsEmpty())
		{
			FSoftObjectPath SoftObjectPath(SaveObjectPath);
			TypeToSave = DuplicateFoliageTypeToNewPackage(SoftObjectPath.GetLongPackageName(), InFoliageType);
		}
	}
	else
	{
		TypeToSave = InFoliageType;
	}

	// Save to disk
	if (TypeToSave)
	{
		TArray<UPackage*> PackagesToSave;
		PackagesToSave.Add(TypeToSave->GetOutermost());
		const bool bCheckDirty = false;
		const bool bPromptToSave = false;
		FEditorFileUtils::EPromptReturnCode ReturnValue = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave);

		if (ReturnValue != FEditorFileUtils::PR_Success)
		{
			TypeToSave = nullptr;
		}
	}

	return TypeToSave;
}

UFoliageType* FFoliageEditUtility::DuplicateFoliageTypeToNewPackage(const FString& InPackageName, UFoliageType* InFoliageType)
{
	UPackage* Package = CreatePackage(*InPackageName);
	UFoliageType* TypeToSave = nullptr;

	// We should not save a copy of this duplicate into the transaction buffer as it's an asset. Save and restore Transactional flag
	EObjectFlags Transactional = InFoliageType->HasAnyFlags(RF_Transactional) ? RF_Transactional : RF_NoFlags;
	
	InFoliageType->ClearFlags(Transactional);
	TypeToSave = Cast<UFoliageType>(StaticDuplicateObject(InFoliageType, Package, *FPackageName::GetLongPackageAssetName(InPackageName)));
	InFoliageType->SetFlags(Transactional);

	TypeToSave->SetFlags(RF_Standalone | RF_Public | Transactional);
	TypeToSave->Modify();

	// Notify the asset registry
	FAssetRegistryModule::AssetCreated(TypeToSave);
	
	return TypeToSave;
}

void FFoliageEditUtility::ReplaceFoliageTypeObject(UWorld* InWorld, UFoliageType* OldType, UFoliageType* NewType)
{
	FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "FoliageMode_ReplaceSettingsObject", "Foliage Editing: Replace Settings Object"));

	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		IFA->Modify();
		TUniqueObj<FFoliageInfo> OldInfo;
		if (IFA->RemoveFoliageInfoAndCopyValue(OldType, OldInfo))
		{
			// Old component needs to go
			if (OldInfo->IsInitialized())
			{
				OldInfo->Uninitialize();
			}

			// Append instances if new foliage type is already exists in this actor
			// Otherwise just replace key entry for instances
			FFoliageInfo* NewInfo = IFA->FindInfo(NewType);
			if (NewInfo)
			{
				NewInfo->Instances.Append(OldInfo->Instances);
				NewInfo->ReallocateClusters(NewType);
			}
			else
			{
				// Make sure if type changes we have proper implementation
				TUniqueObj<FFoliageInfo>& NewFoliageInfo = IFA->AddFoliageInfo(NewType, MoveTemp(OldInfo));
				NewFoliageInfo->ReallocateClusters(NewType);
			}
		}
	}
}


void FFoliageEditUtility::MoveActorFoliageInstancesToLevel(ULevel* InTargetLevel, AActor* InIFA)
{
	// Can't move into a locked level
	if (FLevelUtils::IsLevelLocked(InTargetLevel))
	{
		FNotificationInfo NotificatioInfo(NSLOCTEXT("UnrealEd", "CannotMoveFoliageIntoLockedLevel", "Cannot move the selected foliage into a locked level"));
		NotificatioInfo.bUseThrobber = false;
		FSlateNotificationManager::Get().AddNotification(NotificatioInfo)->SetCompletionState(SNotificationItem::CS_Fail);
		return;
	}

	// Get a world context
	UWorld* World = InTargetLevel->OwningWorld;
	bool PromptToMoveFoliageTypeToAsset = World->GetStreamingLevels().Num() > 0;
	bool ShouldPopulateMeshList = false;

	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MoveSelectedFoliageToSelectedLevel", "Move Selected Foliage to Level"), !GEditor->IsTransactionActive());

	// Iterate over all foliage actors in the world and move selected instances to a foliage actor in the target level
	const int32 NumLevels = World->GetNumLevels();
	for (int32 LevelIdx = 0; LevelIdx < NumLevels; ++LevelIdx)
	{
		ULevel* Level = World->GetLevel(LevelIdx);
		if (Level != InTargetLevel)
		{
			AInstancedFoliageActor* IFA = AInstancedFoliageActor::GetInstancedFoliageActorForLevel(Level, /*bCreateIfNone*/ false);
			
			if (IFA == nullptr)
			{
				continue;
			}
			if (InIFA && IFA != InIFA)
			{
				continue;
			}

			bool CanMoveInstanceType = true;

			TMap<UFoliageType*, FFoliageInfo*> InstancesFoliageType = IFA->GetAllInstancesFoliageType();

			for (auto& MeshPair : InstancesFoliageType)
			{
				if (MeshPair.Key != nullptr && MeshPair.Value != nullptr && !MeshPair.Key->IsAsset())
				{
					// Keep previous selection
					TSet<int32> PreviousSelectionSet = MeshPair.Value->SelectedIndices;
					TArray<int32> PreviousSelectionArray;
					PreviousSelectionArray.Reserve(PreviousSelectionSet.Num());

					for (int32& Value : PreviousSelectionSet)
					{
						PreviousSelectionArray.Add(Value);
					}

					UFoliageType* NewFoliageType = SaveFoliageTypeObject(MeshPair.Key);

					if (NewFoliageType != nullptr && NewFoliageType != MeshPair.Key)
					{
						ReplaceFoliageTypeObject(World, MeshPair.Key, NewFoliageType);
					}

					CanMoveInstanceType = NewFoliageType != nullptr;

					if (NewFoliageType != nullptr)
					{
						// Restore previous selection for move operation
						FFoliageInfo* MeshInfo = IFA->FindInfo(NewFoliageType);
						MeshInfo->SelectInstances(true, PreviousSelectionArray);
					}
				}
			}

			// Update our actor if we saved some foliage type as asset
			if (CanMoveInstanceType)
			{
				IFA = AInstancedFoliageActor::GetInstancedFoliageActorForLevel(Level, /*bCreateIfNone*/ false);
				ensure(IFA != nullptr);

				IFA->MoveAllInstancesToLevel(InTargetLevel);
			}

			if (InIFA)
			{
				return;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
