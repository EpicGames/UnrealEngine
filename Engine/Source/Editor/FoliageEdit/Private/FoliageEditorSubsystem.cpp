// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageEditorSubsystem.h"
#include "CoreGlobals.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "InstancedFoliageActor.h"
#include "LevelUtils.h"
#include "FoliageHelper.h"
#include "ActorPartition/ActorPartitionSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FoliageEditorSubsystem)

#define LOCTEXT_NAMESPACE "FoliageEditorSubsystem"

UFoliageEditorSubsystem::UFoliageEditorSubsystem()
{
}

void UFoliageEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	GEngine->OnActorMoved().AddUObject(this, &UFoliageEditorSubsystem::OnActorMoved);
	GEngine->OnLevelActorDeleted().AddUObject(this, &UFoliageEditorSubsystem::OnActorDeleted);
	GEngine->OnLevelActorOuterChanged().AddUObject(this, &UFoliageEditorSubsystem::OnActorOuterChanged);
	FWorldDelegates::PostApplyLevelOffset.AddUObject(this, &UFoliageEditorSubsystem::OnPostApplyLevelOffset);
	FWorldDelegates::PostApplyLevelTransform.AddUObject(this, &UFoliageEditorSubsystem::OnPostApplyLevelTransform);
	FWorldDelegates::OnPostWorldInitialization.AddUObject(this, &UFoliageEditorSubsystem::OnPostWorldInitialization);

}

void UFoliageEditorSubsystem::Deinitialize()
{
	GEngine->OnActorMoved().RemoveAll(this);
	GEngine->OnLevelActorDeleted().RemoveAll(this);
	GEngine->OnLevelActorOuterChanged().RemoveAll(this);

	FWorldDelegates::PostApplyLevelOffset.RemoveAll(this);
	FWorldDelegates::PostApplyLevelTransform.RemoveAll(this);
	FWorldDelegates::OnPostWorldInitialization.RemoveAll(this);

	Super::Deinitialize();
}

void UFoliageEditorSubsystem::OnPostApplyLevelOffset(ULevel* InLevel, UWorld* InWorld, const FVector& InOffset, bool bWorldShift)
{
	if (InWorld && !InWorld->IsGameWorld())
	{
		for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
		{
			AInstancedFoliageActor* IFA = *It;
			if (IFA->GetLevel() == InLevel)
			{
				IFA->PostApplyLevelOffset(InOffset, bWorldShift);
			}
		}
	}
}

void UFoliageEditorSubsystem::OnPostApplyLevelTransform(ULevel* InLevel, const FTransform& InTransform)
{
	if (InLevel && InLevel->GetWorld() && !InLevel->GetWorld()->IsGameWorld())
	{
		for (TActorIterator<AInstancedFoliageActor> It(InLevel->GetWorld()); It; ++It)
		{
			AInstancedFoliageActor* IFA = *It;
			if (IFA->GetLevel() == InLevel)
			{
				IFA->PostApplyLevelTransform(InTransform);
			}
		}
	}
}

void UFoliageEditorSubsystem::OnPostWorldInitialization(UWorld* InWorld, const UWorld::InitializationValues IVS)
{
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		IFA->DetectFoliageTypeChangeAndUpdate();
	}
}

void UFoliageEditorSubsystem::OnActorMoved(AActor* InActor)
{
	UWorld* InWorld = InActor->GetWorld();
	if (!InWorld || InWorld->IsGameWorld())
	{
		return;
	}

	// Both MoveInstancesForMovedComponent and MoveInstancesForMovedOwnedActors early-out when FLevelUtils::IsApplyingLevelTransform() returns true, doing no work.
	// Add the same check here to avoid expensive TActorIterator operations.
	if (FLevelUtils::IsApplyingLevelTransform())
	{
		return;
	}

	TMap<AInstancedFoliageActor*, TArray<UActorComponent*>> UpdatedIFAsPerComponent;
	
	// Process new instance positions first
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		for (UActorComponent* Component : InActor->GetComponents())
		{
			if (IFA->MoveInstancesForMovedComponent(Component))
			{
				UpdatedIFAsPerComponent.FindOrAdd(IFA).Add(Component);
			}
		}

		if (FFoliageHelper::IsOwnedByFoliage(InActor))
		{
			IFA->MoveInstancesForMovedOwnedActors(InActor);
		}
	}

	// Then update partitioning
	for (const auto& [IFA, Components] : UpdatedIFAsPerComponent)
	{
		for (UActorComponent* Component : Components)
		{
			IFA->UpdateInstancePartitioningForMovedComponent(Component);
		}
	}
}

void UFoliageEditorSubsystem::OnActorOuterChanged(AActor* InActor, UObject* OldOuter)
{
	if (GIsTransacting)
	{
		return;
	}

	ULevel* OldLevel = Cast<ULevel>(OldOuter);

	if (InActor->GetLevel() == OldLevel)
	{
		return;
	}

	if (!FFoliageHelper::IsOwnedByFoliage(InActor))
	{
		return;
	}

	UActorPartitionSubsystem* ActorPartitionSubsystem = OldLevel ? UWorld::GetSubsystem<UActorPartitionSubsystem>(OldLevel->GetWorld()) : nullptr;
	if (ActorPartitionSubsystem && ActorPartitionSubsystem->IsLevelPartition())
	{
		AInstancedFoliageActor* OldIFA = AInstancedFoliageActor::GetInstancedFoliageActorForLevel(OldLevel, false);
		check(OldIFA);

		if (OldIFA)
		{
			OldIFA->UpdateFoliageActorInstance(InActor);
		}
	}
}

void UFoliageEditorSubsystem::OnActorDeleted(AActor* InActor)
{
	if (GIsReinstancing)
	{
		return;
	}
	
	UWorld* InWorld = InActor->GetWorld();
	if (!InWorld || InWorld->IsGameWorld())
	{
		return;
	}

	// Process new instance positions first
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		for (UActorComponent* Component : InActor->GetComponents())
		{
			IFA->DeleteInstancesForComponent(Component);
		}

		if (FFoliageHelper::IsOwnedByFoliage(InActor))
		{
			IFA->DeleteFoliageActorInstance(InActor);
		}
	}
}

#undef LOCTEXT_NAMESPACE

