// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliagePaletteCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "FoliagePaletteCommands"

void FFoliagePaletteCommands::RegisterCommands()
{
	UI_COMMAND(ActivateFoliageType, "Activate", "Sets the selected foliage types in the palette as active (i.e. included in brush actions).", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeactivateFoliageType, "Deactivate", "Sets the selected foliage types in the palette as inactive (i.e. excluded in brush actions).", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveFoliageType, "Remove", "Remove this foliage type from the palette. Removes all associated instances as well.", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));
	UI_COMMAND(ShowFoliageTypeInCB, "Show in Content Browser", "Show asset in Content Browser.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectAllInstances, "Select All Instances", "Select all instances of this foliage type (must be in a selection mode).", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeselectAllInstances, "Deselect All Instances", "Deselect all instances of this foliage type (must be in a selection mode).", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectInvalidInstances, "Select Invalid Instances", "Select all instances of this foliage type that are off ground (must be in a selection mode).", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
