// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FUICommandInfo;

/**
* Foliage Mode Palette Actions
*/
class FFoliagePaletteCommands : public TCommands < FFoliagePaletteCommands >
{

public:
	FFoliagePaletteCommands() : TCommands<FFoliagePaletteCommands>
		(
		"FoliagePalette", // Context name for fast lookup
		NSLOCTEXT("Contexts", "FoliagePalette", "Foliage Palette"), // Localized context name for displaying
		"FoliageEditMode", // Parent
		FAppStyle::GetAppStyleSetName() // Icon Style Set
		)
	{
	}

	/** Initialize commands */
	virtual void RegisterCommands() override;

	/** Foliage Palette Commands */
	TSharedPtr< FUICommandInfo > ActivateFoliageType;
	TSharedPtr< FUICommandInfo > DeactivateFoliageType;
	TSharedPtr< FUICommandInfo > RemoveFoliageType;
	TSharedPtr< FUICommandInfo > ShowFoliageTypeInCB;
	TSharedPtr< FUICommandInfo > SelectAllInstances;
	TSharedPtr< FUICommandInfo > DeselectAllInstances;
	TSharedPtr< FUICommandInfo > SelectInvalidInstances;
};
