// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliagePaletteItem.h"

#include "AssetRegistry/AssetData.h"
#include "AssetThumbnail.h"
#include "AssetToolsModule.h"
#include "Delegates/Delegate.h"
#include "Engine/Blueprint.h"
#include "FoliageEdMode.h"
#include "FoliageType.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Math/Interval.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"
#include "Modules/ModuleManager.h"
#include "SFoliagePalette.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"

class STableViewBase;
class SWidget;

#define LOCTEXT_NAMESPACE "FoliageEd_Mode"

namespace EInstanceCountMagnitude
{
	enum Type
	{
		Thousand,
		Million,
		Billion,
		Max
	};
}

FFoliagePaletteItemModel::FFoliagePaletteItemModel(FFoliageMeshUIInfoPtr InTypeInfo, TSharedRef<SFoliagePalette> InFoliagePalette, TSharedPtr<FAssetThumbnailPool> InThumbnailPool, FEdModeFoliage* InFoliageEditMode)
	: TypeInfo(InTypeInfo)
	, FoliagePalette(InFoliagePalette)
	, FoliageEditMode(InFoliageEditMode)
{
	// Determine the display FName
	DisplayFName = TypeInfo->Settings->GetDisplayFName();
	
	FAssetData AssetData;
	if (!IsBlueprint())
	{
		AssetData = FAssetData(TypeInfo->Settings);
	}
	else
	{
		AssetData = FAssetData(TypeInfo->Settings->GetClass()->GetDefaultObject<UFoliageType>()->GetSource());
	}
	

	int32 MaxThumbnailSize = FoliagePaletteConstants::ThumbnailSizeRange.Max;
	TSharedPtr< FAssetThumbnail > Thumbnail = MakeShareable(new FAssetThumbnail(AssetData, MaxThumbnailSize, MaxThumbnailSize, InThumbnailPool));
	
	FAssetThumbnailConfig ThumbnailConfig;
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
	if (TypeInfo->Settings->IsAsset())
	{
		// Get the Foliage Type asset color
		auto FoliageTypeAssetActions =  AssetToolsModule.Get().GetAssetTypeActionsForClass(TypeInfo->Settings->GetClass());
		if (FoliageTypeAssetActions.IsValid())
		{
			ThumbnailConfig.AssetTypeColorOverride = FoliageTypeAssetActions.Pin()->GetTypeColor();
		}
	}
	else if (TypeInfo->Settings->GetClass()->ClassGeneratedBy)
	{
		// Get the Blueprint asset color
		auto BlueprintAssetActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(UBlueprint::StaticClass());
		if (BlueprintAssetActions.IsValid())
		{
			ThumbnailConfig.AssetTypeColorOverride = BlueprintAssetActions.Pin()->GetTypeColor();
		}
	}

	ThumbnailWidget = Thumbnail->MakeThumbnailWidget(ThumbnailConfig);
}

TSharedPtr<SFoliagePalette> FFoliagePaletteItemModel::GetFoliagePalette() const
{
	return FoliagePalette.Pin();
}

FFoliageMeshUIInfoPtr FFoliagePaletteItemModel::GetTypeUIInfo() const
{
	return TypeInfo;
}

UFoliageType* FFoliagePaletteItemModel::GetFoliageType() const
{
	return TypeInfo->Settings;
}

const FFoliageUISettings& FFoliagePaletteItemModel::GetFoliageUISettings() const
{
	return FoliageEditMode->UISettings;
}

TSharedRef<SWidget> FFoliagePaletteItemModel::GetThumbnailWidget() const
{
	return ThumbnailWidget.ToSharedRef();
}

TSharedRef<SToolTip> FFoliagePaletteItemModel::CreateTooltipWidget() const
{
	return 
		SNew(SToolTip)
		.TextMargin(1.f)
		.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ToolTipBorder"))
		.Visibility(this, &FFoliagePaletteItemModel::GetTooltipVisibility)
		[
			SNew(SBorder)
			.Padding(3.f)
			.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.NonContentBorder"))
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.Padding(FMargin(6.f))
					.HAlign(HAlign_Left)
					.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
					[
						SNew(STextBlock)
						.Text(FText::FromName(DisplayFName))
						.Font(FAppStyle::GetFontStyle("ContentBrowser.TileViewTooltip.NameFont"))
						.HighlightText(this, &FFoliagePaletteItemModel::GetPaletteSearchText)
					]
				]
				
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(0.f, 3.f, 0.f, 0.f))
				[
					
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(0.f, 0.f, 3.f, 0.f))
					[
						SNew(SBorder)
						.Padding(6.f)
						.HAlign(HAlign_Center)
						.Visibility(this, &FFoliagePaletteItemModel::GetTooltipThumbnailVisibility)
						.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
						[
							SNew(SBox)
							.HeightOverride(64.f)
							.WidthOverride(64.f)
							[
								GetThumbnailWidget()
							]
						]
					]

					+ SHorizontalBox::Slot()
					[
						SNew(SBorder)
						.Padding(6.f)
						.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
						[
							SNew(SVerticalBox)

							+ SVerticalBox::Slot()
							.Padding(0, 1)
							.AutoHeight()
							[
								SNew(SHorizontalBox)
								
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(STextBlock)
									.Text(LOCTEXT("SourceAssetTypeHeading", "Source Asset Type: "))
									.ColorAndOpacity(FSlateColor::UseSubduedForeground())
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(STextBlock)
									.Text(this, &FFoliagePaletteItemModel::GetSourceAssetTypeText)
								]
							]

							+ SVerticalBox::Slot()
							.Padding(0, 1)
							.AutoHeight()
							[
								SNew(SHorizontalBox)
								
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(STextBlock)
									.Text(LOCTEXT("InstanceCountHeading", "Instance Count: "))
									.ColorAndOpacity(FSlateColor::UseSubduedForeground())
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(STextBlock)
									.Text(this, &FFoliagePaletteItemModel::GetInstanceCountText, false)
								]
							]
						]
					]					
				]
			]
		];
}

TSharedRef<SCheckBox> FFoliagePaletteItemModel::CreateActivationCheckBox(TAttribute<bool> IsItemWidgetSelected, TAttribute<EVisibility> InVisibility)
{
	return
		SNew(SCheckBox)
		.Padding(0.f)
		.OnCheckStateChanged(this, &FFoliagePaletteItemModel::HandleCheckStateChanged, IsItemWidgetSelected)
		.Visibility(InVisibility)
		.IsChecked(this, &FFoliagePaletteItemModel::GetCheckBoxState)
		.ToolTipText(LOCTEXT("TileCheckboxTooltip", "Check to activate the currently selected types in the palette"));
}

TSharedRef<SButton> FFoliagePaletteItemModel::CreateSaveAssetButton(TAttribute<EVisibility> InVisibility)
{
	return 
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "FoliageEditMode.FloatingButton")
		.Visibility(InVisibility)
		.IsEnabled(this, &FFoliagePaletteItemModel::IsSaveEnabled)
		.OnClicked(this, &FFoliagePaletteItemModel::HandleSaveAsset)
		.ToolTipText(LOCTEXT("SaveButtonToolTip", "Save foliage asset"))
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Top)
		.ContentPadding(0.f)
		.Content()
		[
			SNew(SImage)
	        .ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::Get().GetBrush("Icons.Save"))
		];
}

FName FFoliagePaletteItemModel::GetDisplayFName() const
{
	return DisplayFName;
}

FText FFoliagePaletteItemModel::GetPaletteSearchText() const
{
	if (FoliagePalette.IsValid())
	{
		return FoliagePalette.Pin()->GetSearchText();
	}
	else
	{
		return FText();
	}
}

FText FFoliagePaletteItemModel::GetInstanceCountText(bool bRounded) const
{
	const int32	InstanceCountTotal = TypeInfo->InstanceCountTotal;
	const int32	InstanceCountCurrentLevel = TypeInfo->InstanceCountCurrentLevel;

	if (!bRounded && InstanceCountCurrentLevel != InstanceCountTotal)
	{
		return FText::Format(LOCTEXT("InstanceCount_Total", "{0} ({1})"), FText::AsNumber(InstanceCountCurrentLevel), FText::AsNumber(InstanceCountTotal));
	}
	else if (InstanceCountCurrentLevel >= 1000.f)
	{
		// Note: Instance counts greater than 999 billion (unlikely) will not be formatted properly
		static const FText Suffixes[EInstanceCountMagnitude::Max] =
		{
			LOCTEXT("Suffix_Thousand", "K"),
			LOCTEXT("Suffix_Million", "M"),
			LOCTEXT("Suffix_Billion", "B")
		};

		int32 NumThousands = 0;
		double DisplayValue = InstanceCountCurrentLevel;
		while (DisplayValue >= 1000.f && NumThousands < EInstanceCountMagnitude::Max)
		{
			DisplayValue /= 1000.f;
			NumThousands++;
		}

		// Allow 3 significant figures
		FNumberFormattingOptions Options;
		const int32 NumFractionalDigits = DisplayValue >= 100.f ? 0 : DisplayValue >= 10.f ? 1 : 2;
		Options.SetMaximumFractionalDigits(NumFractionalDigits);

		return FText::Format(LOCTEXT("InstanceCount_CurrentLevel", "{0}{1}"), FText::AsNumber(DisplayValue, &Options), Suffixes[NumThousands - 1]);
	}

	return FText::AsNumber(InstanceCountCurrentLevel);
}

void FFoliagePaletteItemModel::SetTypeActiveInPalette(bool bSetActiveInPalette)
{
	UFoliageType* FoliageType = GetFoliageType();
	if (FoliageType->IsSelected != bSetActiveInPalette)
	{
		FoliageType->Modify(false);
		FoliageType->IsSelected = bSetActiveInPalette;

		if (IsBlueprint())
		{
			FoliageType->GetClass()->ClassGeneratedBy->Modify(false);
			FoliageType->GetClass()->GetDefaultObject<UFoliageType>()->IsSelected = bSetActiveInPalette;
		}
	}
}

bool FFoliagePaletteItemModel::IsActive() const
{
	return TypeInfo->Settings->IsSelected;
}

bool FFoliagePaletteItemModel::IsBlueprint() const
{
	return !TypeInfo->Settings->IsValidLowLevel() || TypeInfo->Settings->GetClass()->ClassGeneratedBy != nullptr;
}

bool FFoliagePaletteItemModel::IsAsset() const
{
	return TypeInfo->Settings->IsAsset();
}

void FFoliagePaletteItemModel::HandleCheckStateChanged(const ECheckBoxState NewCheckedState, TAttribute<bool> IsItemWidgetSelected)
{
	if (!IsItemWidgetSelected.IsSet()) { return; }

	const bool bShouldActivate = NewCheckedState == ECheckBoxState::Checked;
	if (!IsItemWidgetSelected.Get())
	{
		SetTypeActiveInPalette(bShouldActivate);
	}
	else if (FoliagePalette.IsValid())
	{
		FoliagePalette.Pin()->ActivateAllSelectedTypes(bShouldActivate);
	}
}

ECheckBoxState FFoliagePaletteItemModel::GetCheckBoxState() const
{
	return TypeInfo->Settings->IsSelected ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool FFoliagePaletteItemModel::IsSaveEnabled() const
{
	UFoliageType* FoliageType = GetFoliageType();

	// Saving is enabled for non-assets and dirty assets
	return (!FoliageType->IsAsset() || FoliageType->GetOutermost()->IsDirty());
}

FReply FFoliagePaletteItemModel::HandleSaveAsset()
{
	UFoliageType* SavedSettings = FoliageEditMode->SaveFoliageTypeObject(TypeInfo->Settings);
	if (SavedSettings)
	{
		TypeInfo->Settings = SavedSettings;
	}

	return FReply::Handled();
}

EVisibility FFoliagePaletteItemModel::GetTooltipVisibility() const
{
	return (FoliagePalette.IsValid() && FoliagePalette.Pin()->ShouldShowTooltips()) ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
}

EVisibility FFoliagePaletteItemModel::GetTooltipThumbnailVisibility() const
{
	return (FoliagePalette.IsValid() && FoliagePalette.Pin()->IsActiveViewMode(EFoliagePaletteViewMode::Tree)) ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
}

FText FFoliagePaletteItemModel::GetSourceAssetTypeText() const
{
	if (TypeInfo->Settings->IsAsset())
	{
		return LOCTEXT("FoliageTypeAsset", "Foliage Type");
	}
	else if (UBlueprint* FoliageTypeBP = Cast<UBlueprint>(TypeInfo->Settings->GetClass()->ClassGeneratedBy))
	{
		return LOCTEXT("BlueprintClassAsset", "Blueprint Class");
	}
	else
	{
		return LOCTEXT("StaticMeshAsset", "Static Mesh");
	}
}

////////////////////////////////////////////////
// SFoliagePaletteItemTile
////////////////////////////////////////////////

const float SFoliagePaletteItemTile::MinScaleForOverlayItems = 0.2f;

void SFoliagePaletteItemTile::Construct(const FArguments& InArgs, TSharedRef<STableViewBase> InOwnerTableView, TSharedPtr<FFoliagePaletteItemModel>& InModel)
{
	Model = InModel;

	auto IsSelectedGetter = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &SFoliagePaletteItemTile::IsSelected));
	auto CheckBoxVisibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SFoliagePaletteItemTile::GetCheckBoxVisibility));
	auto SaveButtonVisibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SFoliagePaletteItemTile::GetSaveButtonVisibility));

	STableRow<FFoliageMeshUIInfoPtr>::Construct(
		STableRow<FFoliageMeshUIInfoPtr>::FArguments()
		.Style(FAppStyle::Get(), "ContentBrowser.AssetListView.ColumnListTableRow")
		.Padding(1.f)
		.Content()
		[
			SNew(SOverlay)
			.ToolTip(Model->CreateTooltipWidget())
			
			// Thumbnail
			+ SOverlay::Slot()
			[
				SNew(SBorder)
				.Padding(2.f)
				.BorderImage(FAppStyle::GetBrush("ContentBrowser.ThumbnailShadow"))
				.ForegroundColor(FLinearColor::White)
				.ColorAndOpacity(this, &SFoliagePaletteItemTile::GetTileColorAndOpacity)
				[
					Model->GetThumbnailWidget()
				]
			]
			
			// Checkbox
			+ SOverlay::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			.Padding(FMargin(3.f))
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ContentBrowser.ThumbnailShadow"))
				.Padding(3.f)
				[
					Model->CreateActivationCheckBox(IsSelectedGetter, CheckBoxVisibility)
				]
			]

			// Save Button
			+ SOverlay::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Top)
			.Padding(FMargin(2.f))
			[
				Model->CreateSaveAssetButton(SaveButtonVisibility)
			]

			// Instance Count
			+ SOverlay::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(FMargin(6.f, 8.f))
			[
				SNew(STextBlock)
				.Visibility(this, &SFoliagePaletteItemTile::GetInstanceCountVisibility)
				.Text(Model.ToSharedRef(), &FFoliagePaletteItemModel::GetInstanceCountText, true)
				.ShadowOffset(FVector2D(1.f, 1.f))
				.ColorAndOpacity(FLinearColor(.85f, .85f, .85f, 1.f))
			]
		], InOwnerTableView);
}

FLinearColor SFoliagePaletteItemTile::GetTileColorAndOpacity() const
{
	return Model->IsActive() ? FLinearColor::White : FLinearColor(0.5f, 0.5f, 0.5f, 1.f);
}

EVisibility SFoliagePaletteItemTile::GetCheckBoxVisibility() const
{
	return CanShowOverlayItems() && (IsHovered() || (IsSelected() && Model->GetFoliagePalette()->AnySelectedTileHovered())) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SFoliagePaletteItemTile::GetSaveButtonVisibility() const
{
	return IsHovered() && CanShowOverlayItems() && !Model->IsBlueprint() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SFoliagePaletteItemTile::GetInstanceCountVisibility() const
{
	return CanShowOverlayItems() ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SFoliagePaletteItemTile::CanShowOverlayItems() const
{
	return Model->GetFoliageUISettings().GetPaletteThumbnailScale() >= MinScaleForOverlayItems;
}

////////////////////////////////////////////////
// SFoliagePaletteItemRow
////////////////////////////////////////////////

void SFoliagePaletteItemRow::Construct(const FArguments& InArgs, TSharedRef<STableViewBase> InOwnerTableView, TSharedPtr<FFoliagePaletteItemModel>& InModel)
{
	Model = InModel;
	SMultiColumnTableRow<FFoliageMeshUIInfoPtr>::Construct(FSuperRowType::FArguments(), InOwnerTableView);

	SetToolTip(Model->CreateTooltipWidget());
}

TSharedRef<SWidget> SFoliagePaletteItemRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedPtr<SWidget> TableRowContent = SNullWidget::NullWidget;

	if (ColumnName == FoliagePaletteTreeColumns::ColumnID_ToggleActive)
	{
		auto IsSelectedGetter = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &SFoliagePaletteItemTile::IsSelected));
		TableRowContent = Model->CreateActivationCheckBox(IsSelectedGetter);
	} 
	else if (ColumnName == FoliagePaletteTreeColumns::ColumnID_Type)
	{
		TableRowContent =
			SNew(SHorizontalBox)
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
			]
			
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(FText::FromName(Model->GetDisplayFName()))
				.HighlightText(Model.ToSharedRef(), &FFoliagePaletteItemModel::GetPaletteSearchText)
			];
	}
	else if (ColumnName == FoliagePaletteTreeColumns::ColumnID_InstanceCount)
	{
		TableRowContent =
			SNew(SHorizontalBox)
			
			+SHorizontalBox::Slot()
			.Padding(FMargin(10,1,0,1))
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(Model.ToSharedRef(), &FFoliagePaletteItemModel::GetInstanceCountText, true)
			];
	}
	else if (ColumnName == FoliagePaletteTreeColumns::ColumnID_Save)
	{
		auto SaveButtonVisibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SFoliagePaletteItemRow::GetSaveButtonVisibility));

		TableRowContent = Model->CreateSaveAssetButton(SaveButtonVisibility);
	}

	return TableRowContent.ToSharedRef();
}

EVisibility SFoliagePaletteItemRow::GetSaveButtonVisibility() const
{
	return !Model->IsBlueprint() ? EVisibility::Visible : EVisibility::Hidden;
}

#undef LOCTEXT_NAMESPACE
