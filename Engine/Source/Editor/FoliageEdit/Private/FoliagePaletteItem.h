// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "Delegates/IDelegateInstance.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FEdModeFoliage;
class SFoliagePalette;
class STableViewBase;
class SWidget;
class UFoliageType;
struct FFoliageMeshUIInfo;

typedef TSharedPtr<FFoliageMeshUIInfo> FFoliageMeshUIInfoPtr; //should match typedef in FoliageEdMode.h

namespace FoliagePaletteTreeColumns
{
	/** IDs for list columns */
	static const FName ColumnID_ToggleActive("Toggle");
	static const FName ColumnID_Type("Type");
	static const FName ColumnID_InstanceCount("InstanceCount");
	static const FName ColumnID_Save("Save");
};

class FFoliagePaletteItemModel : public TSharedFromThis<FFoliagePaletteItemModel>
{
public:
	FFoliagePaletteItemModel(FFoliageMeshUIInfoPtr InTypeInfo, TSharedRef<SFoliagePalette> InFoliagePalette, TSharedPtr<class FAssetThumbnailPool> InThumbnailPool, FEdModeFoliage* InFoliageEditMode);

	/** @return The foliage palette that contains the item */
	TSharedPtr<SFoliagePalette> GetFoliagePalette() const;

	FFoliageMeshUIInfoPtr GetTypeUIInfo() const;
	UFoliageType* GetFoliageType() const;
	const struct FFoliageUISettings& GetFoliageUISettings() const;

	/** @return The thumbnail widget for this item */
	TSharedRef<SWidget> GetThumbnailWidget() const;

	/** @return The tooltip widget for this item */
	TSharedRef<class SToolTip> CreateTooltipWidget() const;

	/** @return The checkbox widget for activating/deactivating this type in the palette */
	TSharedRef<class SCheckBox> CreateActivationCheckBox(TAttribute<bool> IsItemWidgetSelected, TAttribute<EVisibility> InVisibility = TAttribute<EVisibility>());

	/** @return The save asset button */
	TSharedRef<class SButton> CreateSaveAssetButton(TAttribute<EVisibility> InVisibility = TAttribute<EVisibility>());

	/** Gets the FName version of the displayed name of this item */
	FName GetDisplayFName() const;

	/** Gets the current search filter text */
	FText GetPaletteSearchText() const;

	/** Gets the number of instances of this foliage type  */
	FText GetInstanceCountText(bool bLevelOnly = true) const;

	/** Sets whether this foliage type is active in the palette */
	void SetTypeActiveInPalette(bool bSetActiveInPalette);

	/** Gets whether this foliage type is active in the palette */
	bool IsActive() const;

	/** @return Whether this palette item represents an instance of a foliage type blueprint class */
	bool IsBlueprint() const;

	/** @return Whether this palette item represents a foliage type asset */
	bool IsAsset() const;

private:
	/** Handles the change in activation of the item in the palette */
	void HandleCheckStateChanged(const ECheckBoxState NewCheckedState, TAttribute<bool> IsItemWidgetSelected);

	/** Gets whether the foliage type is active in the palette  */
	ECheckBoxState GetCheckBoxState() const;

	/** Whether the save asset button is enabled */
	bool IsSaveEnabled() const;

	/** Saves the foliage type */
	FReply HandleSaveAsset();

	/** Gets the visibility of the entire tooltip */
	EVisibility GetTooltipVisibility() const;

	/** Gets the visibility of the thumbnail in the tooltip */
	EVisibility GetTooltipThumbnailVisibility() const;

	/** Gets the source asset type text */
	FText GetSourceAssetTypeText() const;
private:
	TSharedPtr<SWidget> ThumbnailWidget;
	FName DisplayFName;
	FFoliageMeshUIInfoPtr TypeInfo;
	TWeakPtr<SFoliagePalette> FoliagePalette;
	FEdModeFoliage* FoliageEditMode;
};

/** A tile representing a foliage type in the palette */
class SFoliagePaletteItemTile : public STableRow<FFoliageMeshUIInfoPtr>
{
public:
	SLATE_BEGIN_ARGS(SFoliagePaletteItemTile) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<STableViewBase> InOwnerTableView, TSharedPtr<FFoliagePaletteItemModel>& InModel);

private:
	FLinearColor GetTileColorAndOpacity() const;
	EVisibility GetCheckBoxVisibility() const;
	EVisibility GetSaveButtonVisibility() const;
	EVisibility GetInstanceCountVisibility() const;
	bool CanShowOverlayItems() const;

private:
	static const float MinScaleForOverlayItems;
	TSharedPtr<FFoliagePaletteItemModel> Model;
};

/** A tree row representing a foliage type in the palette */
class SFoliagePaletteItemRow : public SMultiColumnTableRow<FFoliageMeshUIInfoPtr>
{
public:
	SLATE_BEGIN_ARGS(SFoliagePaletteItemRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<STableViewBase> InOwnerTableView, TSharedPtr<FFoliagePaletteItemModel>& InModel);
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	EVisibility GetSaveButtonVisibility() const;

private:
	TSharedPtr<FFoliagePaletteItemModel> Model;
};
