// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageTypeCustomizationHelpers.h"

#include "Components/PrimitiveComponent.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "FoliageType.h"
#include "HAL/Platform.h"
#include "IDetailPropertyRow.h"
#include "Misc/AssertionMacros.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "UObject/EnumProperty.h"
#include "UObject/UnrealType.h"

void FFoliageTypeCustomizationHelpers::ModifyFoliagePropertyRow(IDetailPropertyRow* PropertyRow, const TAttribute<EVisibility>& InVisibility, const TAttribute<bool>& InEnabled)
{
	if (PropertyRow)
	{
		if (InVisibility.IsSet())
		{
			PropertyRow->Visibility(InVisibility);
		}
		if (InEnabled.IsSet())
		{
			PropertyRow->IsEnabled(InEnabled);
		}
	}
}

void FFoliageTypeCustomizationHelpers::AddBodyInstanceProperties(IDetailLayoutBuilder& LayoutBuilder)
{
	static const FName PhysicsName("Physics");
	IDetailCategoryBuilder& PhysicsCategory = LayoutBuilder.EditCategory(PhysicsName);

	TSharedRef<IPropertyHandle> BodyInstanceHandle = LayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPrimitiveComponent, BodyInstance));

	PhysicsCategory.AddProperty(BodyInstanceHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, WalkableSlopeOverride)));
	PhysicsCategory.AddProperty(BodyInstanceHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, PhysMaterialOverride)));
}

void FFoliageTypeCustomizationHelpers::HideFoliageCategory(IDetailLayoutBuilder& DetailLayoutBuilder, FName CategoryName)
{
	TArray<TSharedRef<IPropertyHandle>> CategoryProperties;
	DetailLayoutBuilder.EditCategory(CategoryName).GetDefaultProperties(CategoryProperties, true, true);

	for (auto& PropertyHandle : CategoryProperties)
	{
		DetailLayoutBuilder.HideProperty(PropertyHandle);
	}
}

void FFoliageTypeCustomizationHelpers::BindHiddenPropertyVisibilityGetter(const TSharedPtr<IPropertyHandle>& PropertyHandle, TAttribute<EVisibility>::FGetter& OutVisibilityGetter)
{
	// Determine the conditions for showing this property. 
	// Note that any special cases can override the visibility attribute by using ModifyFoliageProperty
	
	if (PropertyHandle->GetProperty()->IsA<FBoolProperty>())
	{
		// If hidden behind a bool, show when that bool is true
		OutVisibilityGetter.BindLambda([=]
		{
			bool bState = 0;
			auto Result = PropertyHandle->GetValue(bState);
			return bState && Result == FPropertyAccess::Success ? EVisibility::Visible : EVisibility::Collapsed;
		});
	}
	else if (PropertyHandle->GetProperty()->IsA<FByteProperty>() || PropertyHandle->GetProperty()->IsA<FEnumProperty>())
	{
		// If hidden behind a byte/enum, assume that 0 = disabled and show if nonzero
		OutVisibilityGetter.BindLambda([=]
		{
			uint8 Value = 0;
			auto Result = PropertyHandle->GetValue(Value);
			return  Value != 0 && Result == FPropertyAccess::Success ? EVisibility::Visible : EVisibility::Collapsed;
		});
	}
	else
	{
		auto ArrayPropHandle = PropertyHandle->AsArray();
		if (ArrayPropHandle.IsValid())
		{
			// If hidden behind an array, show when the array isn't empty
			OutVisibilityGetter.BindLambda([=]
			{
				uint32 NumElements = 0;
				ArrayPropHandle->GetNumElements(NumElements);
				return  NumElements > 0 ? EVisibility::Visible : EVisibility::Collapsed;
			});
		}
	}
}

EVisibility FFoliageTypeCustomizationHelpers::GetScaleAxisVisibility(EAxis::Type Axis, const TSharedPtr<IPropertyHandle> ScalingPropertyHandle)
{
	if (ScalingPropertyHandle.IsValid())
	{
		uint8 ScalingValue;
		if (ScalingPropertyHandle->GetValue(ScalingValue) == FPropertyAccess::Success)
		{
			switch (Axis)
			{
			case EAxis::X:
				return EVisibility::Visible;

			case EAxis::Y:
			case EAxis::Z:
				return (ScalingValue == (uint8)EFoliageScaling::Uniform) ? EVisibility::Collapsed : EVisibility::Visible;
			}
		}
	}

	return EVisibility::Visible;
}
