// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Visibility.h"
#include "Math/Axis.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IDetailLayoutBuilder;
class IDetailPropertyRow;
class IPropertyHandle;

class FFoliageTypeCustomizationHelpers
{
public:

	/** Modifies the visibility and enabled attributes of an existing property row */
	static void ModifyFoliagePropertyRow(IDetailPropertyRow* PropertyRow, const TAttribute<EVisibility>& InVisibility, const TAttribute<bool>& InEnabled);

	static void AddBodyInstanceProperties(IDetailLayoutBuilder& LayoutBuilder);
		
	/** Hides all properties in the given category */
	static void HideFoliageCategory(IDetailLayoutBuilder& DetailLayoutBuilder, FName CategoryName);

	/** Binds the appropriate visibility getter for the hidden property */
	static void BindHiddenPropertyVisibilityGetter(const TSharedPtr<IPropertyHandle>& PropertyHandle, TAttribute<EVisibility>::FGetter& OutVisibilityGetter);

	static EVisibility GetScaleAxisVisibility(EAxis::Type Axis, const TSharedPtr<IPropertyHandle> ScalingPropertyHandle);
};
