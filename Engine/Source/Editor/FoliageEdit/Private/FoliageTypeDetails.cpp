// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageTypeDetails.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "FoliageTypeCustomizationHelpers.h"
#include "HAL/PlatformCrt.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "Settings/EditorExperimentalSettings.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

TSharedRef<IDetailCustomization> FFoliageTypeDetails::MakeInstance()
{
	return MakeShareable(new FFoliageTypeDetails());
}

void CustomizePropertyRowVisibility(IDetailLayoutBuilder& LayoutBuilder, const TSharedRef<IPropertyHandle>& PropertyHandle, IDetailPropertyRow& PropertyRow)
{
	// Properties with a HideBehind property specified should only be shown if that property is true, non-zero, or not empty
	static const FName HideBehindName("HideBehind");
	if (FProperty* Property = PropertyHandle->GetProperty())
	{
		if (Property->HasMetaData(HideBehindName))
		{
			TSharedPtr<IPropertyHandle> HiddenBehindPropertyHandle = LayoutBuilder.GetProperty(*Property->GetMetaData(HideBehindName));
			if (HiddenBehindPropertyHandle.IsValid() && HiddenBehindPropertyHandle->IsValidHandle())
			{
				TAttribute<EVisibility>::FGetter VisibilityGetter;
				FFoliageTypeCustomizationHelpers::BindHiddenPropertyVisibilityGetter(HiddenBehindPropertyHandle, VisibilityGetter);

				PropertyRow.Visibility(TAttribute<EVisibility>::Create(VisibilityGetter));
			}
		}
	}
}

void AddSubcategoryProperties(IDetailLayoutBuilder& LayoutBuilder, const FName CategoryName)
{
	IDetailCategoryBuilder& CategoryBuilder = LayoutBuilder.EditCategory(CategoryName);

	TArray<TSharedRef<IPropertyHandle>> CategoryProperties;
	CategoryBuilder.GetDefaultProperties(CategoryProperties, true, true);

	//Build map of subcategories to properties
	static const FName SubcategoryName("Subcategory");
	TMap<FString, TArray<TSharedRef<IPropertyHandle>> > SubcategoryPropertiesMap;
	for (auto& PropertyHandle : CategoryProperties)
	{
		if (FProperty* Property = PropertyHandle->GetProperty())
		{
			if (Property->HasMetaData(SubcategoryName))
			{
				const FString& Subcategory = Property->GetMetaData(SubcategoryName);
				TArray<TSharedRef<IPropertyHandle>>& PropertyHandles = SubcategoryPropertiesMap.FindOrAdd(Subcategory);
				PropertyHandles.Add(PropertyHandle);
			}
			else
			{
				// The property is not in a subcategory, so add it now
				auto& PropertyRow = CategoryBuilder.AddProperty(PropertyHandle);
				CustomizePropertyRowVisibility(LayoutBuilder, PropertyHandle, PropertyRow);
			}
		}
	}

	//Add subgroups
	for (auto Iter = SubcategoryPropertiesMap.CreateConstIterator(); Iter; ++Iter)
	{
		const FString &GroupString = Iter.Key();
		const TArray<TSharedRef<IPropertyHandle>>& PropertyHandles = Iter.Value();
		IDetailGroup& Group = CategoryBuilder.AddGroup(FName(*GroupString), FText::FromString(GroupString));
		for (auto& PropertyHandle : PropertyHandles)
		{
			auto& PropertyRow = Group.AddPropertyRow(PropertyHandle);
			CustomizePropertyRowVisibility(LayoutBuilder, PropertyHandle, PropertyRow);
		}
	}
}

void FFoliageTypeDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const static FName ReapplyName("Reapply");
	const static FName PaintingName("Painting");
	const static FName PlacementName("Placement");
	const static FName ProceduralName("Procedural");
	const static FName InstanceSettingsName("InstanceSettings");

	FFoliageTypeCustomizationHelpers::HideFoliageCategory(DetailBuilder, ReapplyName);

	AddSubcategoryProperties(DetailBuilder, PaintingName);
	AddSubcategoryProperties(DetailBuilder, PlacementName);

	// If enabled, show the properties for procedural placement
	if (GetDefault<UEditorExperimentalSettings>()->bProceduralFoliage)
	{
		AddSubcategoryProperties(DetailBuilder, ProceduralName);
	}
	else
	{
		FFoliageTypeCustomizationHelpers::HideFoliageCategory(DetailBuilder, ProceduralName);
	}
	
	AddSubcategoryProperties(DetailBuilder, InstanceSettingsName);

	FFoliageTypeCustomizationHelpers::AddBodyInstanceProperties(DetailBuilder);
}
