// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageTypeFactory.h"

#include "FoliageType_Actor.h"
#include "FoliageType_InstancedStaticMesh.h"
#include "Internationalization/Internationalization.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FoliageTypeFactory)

class FFeedbackContext;
class UClass;
class UObject;

#define LOCTEXT_NAMESPACE "FoliageTypeFactory"

UFoliageType_InstancedStaticMeshFactory::UFoliageType_InstancedStaticMeshFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UFoliageType_InstancedStaticMesh::StaticClass();
}

UObject* UFoliageType_InstancedStaticMeshFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UFoliageType_InstancedStaticMesh>(InParent, Class, Name, Flags | RF_Transactional);
}

FString UFoliageType_InstancedStaticMeshFactory::GetDefaultNewAssetName() const
{
	return UFoliageType_InstancedStaticMesh::StaticClass()->GetDefaultObject<UFoliageType_InstancedStaticMesh>()->GetDefaultNewAssetName();
}

FText UFoliageType_InstancedStaticMeshFactory::GetToolTip() const
{
	return LOCTEXT("FoliageTypeStaticMeshToolTip", "Static Mesh Foliage is a foliage type that will use mesh instancing and is optimal for non-interactive foliage.");
}

UFoliageType_ActorFactory::UFoliageType_ActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UFoliageType_Actor::StaticClass();
}

UObject* UFoliageType_ActorFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UFoliageType_Actor>(InParent, Class, Name, Flags | RF_Transactional);
}

FString UFoliageType_ActorFactory::GetDefaultNewAssetName() const
{
	return UFoliageType_Actor::StaticClass()->GetDefaultObject<UFoliageType_Actor>()->GetDefaultNewAssetName();
}

FText UFoliageType_ActorFactory::GetToolTip() const
{
	return LOCTEXT("FoliageTypeActorToolTip", "Actor Foliage is a foliage type that will place blueprint/native actor instances. The cost of painting this foliage is the same as adding actors in a scene.");
}

#undef LOCTEXT_NAMESPACE
