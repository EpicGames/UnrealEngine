// Copyright Epic Games, Inc. All Rights Reserved.

/**
* Factory for FoliageType assets
*/

#pragma once

#include "Containers/UnrealString.h"
#include "Factories/Factory.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "FoliageTypeFactory.generated.h"

class FFeedbackContext;
class UClass;
class UObject;

UCLASS()
class UFoliageType_InstancedStaticMeshFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FString GetDefaultNewAssetName() const override;
	virtual FText GetToolTip() const override;
	// End of UFactory interface
};

UCLASS()
class UFoliageType_ActorFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FString GetDefaultNewAssetName() const override;
	virtual FText GetToolTip() const override;
	// End of UFactory interface
};