// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageTypeObjectCustomization.h"

#include "AssetRegistry/AssetData.h"
#include "Blueprint/BlueprintSupport.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "DetailWidgetRow.h"
#include "FoliageType_Actor.h"
#include "FoliageType_InstancedStaticMesh.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/Optional.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class IDetailChildrenBuilder;
class UClass;

void FFoliageTypeObjectCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> FoliageTypeHandle = PropertyHandle->GetChildHandle("FoliageTypeObject");
	
	// Only allow foliage type assets to be created (i.e. don't show all the factories for blueprints)
	TArray<const UClass*> SupportedClasses;
	SupportedClasses.Add(UFoliageType_InstancedStaticMesh::StaticClass());
	SupportedClasses.Add(UFoliageType_Actor::StaticClass());

	HeaderRow
		.NameContent()
		[
			FoliageTypeHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(250.f)
		.MaxDesiredWidth(0.f)
		[
			SNew(SObjectPropertyEntryBox)
			.PropertyHandle(FoliageTypeHandle)
			.ThumbnailPool(CustomizationUtils.GetThumbnailPool())
			.NewAssetFactories(PropertyCustomizationHelpers::GetNewAssetFactoriesForClasses(SupportedClasses))
			.OnShouldFilterAsset(this, &FFoliageTypeObjectCustomization::OnShouldFilterAsset)
		];

		//PropertyCustomizationHelpers::MakeInsertDeleteDuplicateButton <-- not sure how to get that to show up properly, since the struct doesn't know it's an array
}

void FFoliageTypeObjectCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

bool FFoliageTypeObjectCustomization::OnShouldFilterAsset(const FAssetData& AssetData) const
{
	// If the asset is a BP class that doesn't inherit from UFoliageType, hide it
	const FString ParentClassName = AssetData.GetTagValueRef<FString>(FBlueprintTags::ParentClassPath);
	if (!ParentClassName.IsEmpty() && !ParentClassName.Contains(TEXT("FoliageType_InstancedStaticMesh")) && !ParentClassName.Contains(TEXT("FoliageType_Actor")))
	{
		return true;
	}

	return false;
}
