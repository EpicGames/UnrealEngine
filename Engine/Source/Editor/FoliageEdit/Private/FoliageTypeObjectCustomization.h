// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;
struct FAssetData;

class FFoliageTypeObjectCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FFoliageTypeObjectCustomization());
	}

	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	bool OnShouldFilterAsset(const FAssetData& AssetData) const;
};
