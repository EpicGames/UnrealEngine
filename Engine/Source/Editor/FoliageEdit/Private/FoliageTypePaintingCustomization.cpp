// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageTypePaintingCustomization.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Customizations/MobilityCustomization.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "FoliageEdMode.h"
#include "FoliageType.h"
#include "FoliageTypeCustomizationHelpers.h"
#include "HAL/PlatformCrt.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Styling/SlateTypes.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "FoliageEd_Mode"

DECLARE_DELEGATE_RetVal(EVisibility, FFoliageVisibilityDelegate);

/////////////////////////////////////////////////////
// FFoliageTypePaintingCustomization 
TSharedRef<IDetailCustomization> FFoliageTypePaintingCustomization::MakeInstance(FEdModeFoliage* InFoliageEditMode)
{
	TSharedRef<FFoliageTypePaintingCustomization> Instance = MakeShareable(new FFoliageTypePaintingCustomization(InFoliageEditMode));
	return Instance;
}

FFoliageTypePaintingCustomization::FFoliageTypePaintingCustomization(FEdModeFoliage* InFoliageEditMode)
	:FoliageEditMode(InFoliageEditMode)
{
}

void FFoliageTypePaintingCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	// Hide categories we are not going to customize
	FFoliageTypeCustomizationHelpers::HideFoliageCategory(DetailLayoutBuilder, "Procedural");
	FFoliageTypeCustomizationHelpers::HideFoliageCategory(DetailLayoutBuilder, "Reapply");
	
	// Show all the properties with a reapply condition or that depend on another variable to be relevant
	TMap<const FName, IDetailPropertyRow*> PropertyRowsByName;
	ShowFoliagePropertiesForCategory(DetailLayoutBuilder, "Painting", PropertyRowsByName);
	ShowFoliagePropertiesForCategory(DetailLayoutBuilder, "Placement", PropertyRowsByName);
	ShowFoliagePropertiesForCategory(DetailLayoutBuilder, "InstanceSettings", PropertyRowsByName);
	FFoliageTypeCustomizationHelpers::AddBodyInstanceProperties(DetailLayoutBuilder);

	// Density adjustment factor should only be visible when reapplying
	FFoliageTypeCustomizationHelpers::ModifyFoliagePropertyRow(*PropertyRowsByName.Find(GET_MEMBER_NAME_CHECKED(UFoliageType, DensityAdjustmentFactor)),
		TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FFoliageTypePaintingCustomization::GetReapplyModeVisibility)),
		TAttribute<bool>());

	// Set the scale visibility attribute for each axis
	Scaling = DetailLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFoliageType, Scaling));
	ReapplyScaling = DetailLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFoliageType, ReapplyScaling));
	FFoliageTypeCustomizationHelpers::ModifyFoliagePropertyRow(*PropertyRowsByName.Find(GET_MEMBER_NAME_CHECKED(UFoliageType, ScaleX)),
		TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FFoliageTypePaintingCustomization::GetScaleVisibility, EAxis::X)),
		TAttribute<bool>());

	FFoliageTypeCustomizationHelpers::ModifyFoliagePropertyRow(*PropertyRowsByName.Find(GET_MEMBER_NAME_CHECKED(UFoliageType, ScaleY)),
		TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FFoliageTypePaintingCustomization::GetScaleVisibility, EAxis::Y)),
		TAttribute<bool>());

	FFoliageTypeCustomizationHelpers::ModifyFoliagePropertyRow(*PropertyRowsByName.Find(GET_MEMBER_NAME_CHECKED(UFoliageType, ScaleZ)),
		TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FFoliageTypePaintingCustomization::GetScaleVisibility, EAxis::Z)),
		TAttribute<bool>());
}

void FFoliageTypePaintingCustomization::ShowFoliagePropertiesForCategory(IDetailLayoutBuilder& DetailLayoutBuilder, const FName CategoryName, TMap<const FName, IDetailPropertyRow*>& OutDetailRowsByPropertyName)
{
	// Properties that have a ReapplyCondition should be disabled behind the specified property when in reapply mode
	static const FName ReapplyConditionKey("ReapplyCondition");
	
	// Properties with a HideBehind property specified should only be shown if that property is true, non-zero, or not empty
	static const FName HideBehindKey("HideBehind");

	// Mobility property name
	static const FName MobilityName("Mobility");
	
	IDetailCategoryBuilder& CategoryBuilder = DetailLayoutBuilder.EditCategory(CategoryName);
	TArray<TSharedRef<IPropertyHandle>> CategoryProperties;
	CategoryBuilder.GetDefaultProperties(CategoryProperties, true, true);

	// Determine whether each property should be shown and how
	for (auto& PropertyHandle : CategoryProperties)
	{
		bool bShowingProperty = false;
		if (FProperty* Property = PropertyHandle->GetProperty())
		{
			if (Property->GetFName() == MobilityName)
			{
				CategoryBuilder.AddCustomBuilder(MakeShared<FMobilityCustomization>(DetailLayoutBuilder.GetProperty(MobilityName), FMobilityCustomization::StationaryMobilityBitMask, false), false);
			}
			else
			{
				// Check to see if this property can be reapplied
				TSharedPtr<IPropertyHandle> ReapplyConditionPropertyHandle = DetailLayoutBuilder.GetProperty(*Property->GetMetaData(ReapplyConditionKey));
				if (ReapplyConditionPropertyHandle.IsValid() && ReapplyConditionPropertyHandle->IsValidHandle())
				{
					// Create a custom entry that allows explicit enabling/disabling of the property when reapplying
					TSharedPtr<IPropertyHandle> PropertyHandlePtr = PropertyHandle;
					OutDetailRowsByPropertyName.FindOrAdd(PropertyHandle->GetProperty()->GetFName()) =
						&AddFoliageProperty(CategoryBuilder, PropertyHandlePtr, ReapplyConditionPropertyHandle, TAttribute<EVisibility>(), TAttribute<bool>());
				}
				else
				{
					TSharedPtr<IPropertyHandle> InvalidProperty;
					TSharedPtr<IPropertyHandle> PropertyHandlePtr = PropertyHandle;

					// Check to see if this property is hidden behind another
					TSharedPtr<IPropertyHandle> HiddenBehindPropertyHandle = DetailLayoutBuilder.GetProperty(*Property->GetMetaData(HideBehindKey));
					if (HiddenBehindPropertyHandle.IsValid() && HiddenBehindPropertyHandle->IsValidHandle())
					{
						TAttribute<bool> IsEnabledAttribute;
						ReapplyConditionPropertyHandle = DetailLayoutBuilder.GetProperty(*HiddenBehindPropertyHandle->GetProperty()->GetMetaData(ReapplyConditionKey));
						if (ReapplyConditionPropertyHandle.IsValid() && ReapplyConditionPropertyHandle->IsValidHandle())
						{
							// If the property this is hidden behind has a reapply condition, disable this when the condition is false
							IsEnabledAttribute = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FFoliageTypePaintingCustomization::IsReapplyPropertyEnabled, ReapplyConditionPropertyHandle));
						}

						TAttribute<EVisibility> VisibilityAttribute;
						GetHiddenPropertyVisibility(HiddenBehindPropertyHandle, !IsEnabledAttribute.IsSet(), VisibilityAttribute);

						OutDetailRowsByPropertyName.FindOrAdd(PropertyHandle->GetProperty()->GetFName()) =
							&AddFoliageProperty(CategoryBuilder, PropertyHandlePtr, InvalidProperty, VisibilityAttribute, IsEnabledAttribute);
					}
					else
					{
						// This property cannot be reapplied and isn't hidden behind anything, so show it whenever the reapply tool isn't active
						OutDetailRowsByPropertyName.FindOrAdd(PropertyHandle->GetProperty()->GetFName()) =
							&AddFoliageProperty(CategoryBuilder, PropertyHandlePtr, InvalidProperty,
							TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FFoliageTypePaintingCustomization::GetNonReapplyPropertyVisibility)),
							TAttribute<bool>());
					}
				}
			}
		}
	}
}

IDetailPropertyRow& FFoliageTypePaintingCustomization::AddFoliageProperty(
		IDetailCategoryBuilder& Category,
		TSharedPtr<IPropertyHandle>& Property, 
		TSharedPtr<IPropertyHandle>& ReapplyProperty,
		const TAttribute<EVisibility>& InVisibility,
		const TAttribute<bool>& InEnabled)
{
	IDetailPropertyRow& PropertyRow = Category.AddProperty(Property);

	if (ReapplyProperty.IsValid())
	{
		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		FDetailWidgetRow	Row;
		PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

		auto IsEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FFoliageTypePaintingCustomization::IsReapplyPropertyEnabled, ReapplyProperty));
		ValueWidget->SetEnabled(IsEnabled);
		NameWidget->SetEnabled(IsEnabled);

		PropertyRow.CustomWidget(true)
		.NameContent()
		.MinDesiredWidth(Row.NameWidget.MinWidth)
		.MaxDesiredWidth(Row.NameWidget.MaxWidth)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FFoliageTypePaintingCustomization::GetReapplyPropertyState, ReapplyProperty)
				.OnCheckStateChanged(this, &FFoliageTypePaintingCustomization::OnReapplyPropertyStateChanged, ReapplyProperty)
				.Visibility(this, &FFoliageTypePaintingCustomization::GetReapplyModeVisibility)
				.ToolTipText(ReapplyProperty->GetToolTipText())
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				NameWidget.ToSharedRef()
			]
		]
		.ValueContent()
		.MinDesiredWidth(Row.ValueWidget.MinWidth)
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		[
			ValueWidget.ToSharedRef()
		];
	}
	else 
	{
		if (InEnabled.IsSet())
		{
			PropertyRow.IsEnabled(InEnabled);
		}
	}

	if (InVisibility.IsSet())
	{
		PropertyRow.Visibility(InVisibility);
	}
	
	return PropertyRow;
}

void FFoliageTypePaintingCustomization::GetHiddenPropertyVisibility(const TSharedPtr<IPropertyHandle>& PropertyHandle, bool bHideInReapplyTool, TAttribute<EVisibility>& OutVisibility) const
{
	TAttribute<EVisibility>::FGetter VisibilityGetter;
	FFoliageTypeCustomizationHelpers::BindHiddenPropertyVisibilityGetter(PropertyHandle, VisibilityGetter);

	if (bHideInReapplyTool)
	{
		// In addition to hiding it it behind the given property, only show this in the reapply tool
		OutVisibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([this, VisibilityGetter]
		{
			if (!FoliageEditMode->UISettings.GetReapplyToolSelected() && VisibilityGetter.IsBound())
			{
				const EVisibility ReturnVal = VisibilityGetter.Execute();
				return ReturnVal;
			}
			return EVisibility::Collapsed;
		}));
	}
	else
	{
		OutVisibility.Bind(VisibilityGetter);
	}
}

EVisibility FFoliageTypePaintingCustomization::GetScaleVisibility(EAxis::Type Axis) const
{
	// In reapply mode we only want to show these if scaling is being reapplied
	if (IsReapplyPropertyEnabled(ReapplyScaling) || !FoliageEditMode->UISettings.GetReapplyToolSelected())
	{
		return FFoliageTypeCustomizationHelpers::GetScaleAxisVisibility(Axis, Scaling);
	}

	return EVisibility::Collapsed;
}

EVisibility FFoliageTypePaintingCustomization::GetReapplyModeVisibility() const
{
	return FoliageEditMode->UISettings.GetReapplyToolSelected() ? EVisibility::Visible : EVisibility::Collapsed;
}

ECheckBoxState FFoliageTypePaintingCustomization::GetReapplyPropertyState(TSharedPtr<IPropertyHandle> ReapplyProperty) const
{
	bool bState;
	if (ReapplyProperty->GetValue(bState) == FPropertyAccess::Success)
	{
		return bState ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
		
	return ECheckBoxState::Undetermined;
}

void FFoliageTypePaintingCustomization::OnReapplyPropertyStateChanged(ECheckBoxState CheckState, TSharedPtr<IPropertyHandle> ReapplyProperty)
{
	if (CheckState != ECheckBoxState::Undetermined)
	{
		ReapplyProperty->SetValue(CheckState == ECheckBoxState::Checked);
	}
}

bool FFoliageTypePaintingCustomization::IsReapplyPropertyEnabled(TSharedPtr<IPropertyHandle> ReapplyProperty) const
{
	if (FoliageEditMode->UISettings.GetReapplyToolSelected())
	{
		bool bState;
		if (ReapplyProperty->GetValue(bState) == FPropertyAccess::Success)
		{
			return bState;
		}
	}

	return true;
}

EVisibility FFoliageTypePaintingCustomization::GetNonReapplyPropertyVisibility() const
{
	// Visible if the reapply tool is not active
	return !FoliageEditMode->UISettings.GetReapplyToolSelected() ? EVisibility::Visible : EVisibility::Collapsed;
}


#undef LOCTEXT_NAMESPACE
