// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Layout/Visibility.h"
#include "Math/Axis.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FEdModeFoliage;
class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class IDetailPropertyRow;
class IPropertyHandle;

enum class ECheckBoxState : uint8;
template< typename ObjectType > class TAttribute;

/////////////////////////////////////////////////////
// FFoliageTypePaintingCustomization
class FFoliageTypePaintingCustomization : public IDetailCustomization
{
public:
	// Makes a new instance of this detail layout class
	static TSharedRef<IDetailCustomization> MakeInstance(FEdModeFoliage* InFoliageEditMode);

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface
private:
	FFoliageTypePaintingCustomization(FEdModeFoliage* InFoliageEditMode);
	
	EVisibility GetScaleVisibility(EAxis::Type Axis) const;
	EVisibility GetNonReapplyPropertyVisibility() const;
	EVisibility GetReapplyModeVisibility() const;

	ECheckBoxState GetReapplyPropertyState(TSharedPtr<IPropertyHandle> ReapplyProperty) const;
	void OnReapplyPropertyStateChanged(ECheckBoxState CheckState, TSharedPtr<IPropertyHandle> ReapplyProperty);
	bool IsReapplyPropertyEnabled(TSharedPtr<IPropertyHandle> ReapplyProperty) const;

	/** Adds a property that can optionally be hidden behind another when the reapply tool is active */
	IDetailPropertyRow& AddFoliageProperty(
		IDetailCategoryBuilder& Category,
		TSharedPtr<IPropertyHandle>& Property, 
		TSharedPtr<IPropertyHandle>& ReapplyProperty,
		const TAttribute<EVisibility>& InVisibility,
		const TAttribute<bool>& InEnabled);

	void ShowFoliagePropertiesForCategory(IDetailLayoutBuilder& DetailLayoutBuilder, const FName CategoryName, TMap<const FName, IDetailPropertyRow*>& OutDetailRowsByPropertyName);

	/** Gets the visibility attribute for property that is hidden behind another */
	void GetHiddenPropertyVisibility(const TSharedPtr<IPropertyHandle>& PropertyHandle, bool bHideInReapplyTool, TAttribute<EVisibility>& OutVisibility) const;

private:
	/** Pointer to the foliage edit mode. */
	FEdModeFoliage* FoliageEditMode;

	TSharedPtr<IPropertyHandle> Scaling;
	TSharedPtr<IPropertyHandle> ReapplyScaling;
};
