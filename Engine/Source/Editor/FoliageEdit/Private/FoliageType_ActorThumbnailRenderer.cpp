// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageType_ActorThumbnailRenderer.h"

#include "FoliageType_Actor.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FoliageType_ActorThumbnailRenderer)

bool UFoliageType_ActorThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	UFoliageType_Actor* FoliageType = Cast<UFoliageType_Actor>(Object);
	if (FoliageType == nullptr || FoliageType->ActorClass == nullptr)
	{
		return false;
	}

	return Super::CanVisualizeAsset(FoliageType->ActorClass->ClassGeneratedBy);
}

void UFoliageType_ActorThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UFoliageType_Actor* FoliageType = Cast<UFoliageType_Actor>(Object);
	if (FoliageType == nullptr || FoliageType->ActorClass == nullptr)
	{
		return;
	}

	Super::Draw(FoliageType->ActorClass->ClassGeneratedBy, X, Y, Width, Height, RenderTarget, Canvas, bAdditionalViewFamily);
}


