// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageType_ISMThumbnailRenderer.h"

#include "Containers/Array.h"
#include "FoliageType_InstancedStaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "ShowFlags.h"
#include "Templates/Casts.h"
#include "ThumbnailHelpers.h"
#include "ThumbnailRendering/ThumbnailRenderer.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FoliageType_ISMThumbnailRenderer)

bool UFoliageType_ISMThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	UFoliageType_InstancedStaticMesh* FoliageType = Cast<UFoliageType_InstancedStaticMesh>(Object);
	return FoliageType && FoliageType->GetStaticMesh();
}

void UFoliageType_ISMThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UFoliageType_InstancedStaticMesh* FoliageType = Cast<UFoliageType_InstancedStaticMesh>(Object);
	if (FoliageType && FoliageType->GetStaticMesh())
	{
		if (ThumbnailScene == nullptr)
		{
			ThumbnailScene = new FStaticMeshThumbnailScene();
		}

		ThumbnailScene->SetStaticMesh(FoliageType->GetStaticMesh());
		ThumbnailScene->SetOverrideMaterials(FoliageType->OverrideMaterials);
		ThumbnailScene->GetScene()->UpdateSpeedTreeWind(0.0);

		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game))
			.SetTime(UThumbnailRenderer::GetTime())
			.SetAdditionalViewFamily(bAdditionalViewFamily));

		ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
		ViewFamily.EngineShowFlags.MotionBlur = 0;
		ViewFamily.EngineShowFlags.LOD = 0;

		RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
		ThumbnailScene->SetStaticMesh(nullptr);
		ThumbnailScene->SetOverrideMaterials(TArray<class UMaterialInterface*>());
	}
}

void UFoliageType_ISMThumbnailRenderer::BeginDestroy()
{
	if (ThumbnailScene != nullptr)
	{
		delete ThumbnailScene;
		ThumbnailScene = nullptr;
	}

	Super::BeginDestroy();
}
