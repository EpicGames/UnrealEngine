// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageType_InstancedStaticMeshPaintingCustomization.h"

#include "AssetRegistry/AssetData.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/StaticMesh.h"
#include "FoliageEdMode.h"
#include "FoliageType.h"
#include "FoliageType_InstancedStaticMesh.h"
#include "HAL/PlatformCrt.h"
#include "IDetailPropertyRow.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "PropertyCustomizationHelpers.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class IPropertyHandle;
class SWidget;

#define LOCTEXT_NAMESPACE "FoliageEd_Mode"

DECLARE_DELEGATE_RetVal(EVisibility, FFoliageVisibilityDelegate);

/////////////////////////////////////////////////////
// FFoliageTypePaintingCustomization 
TSharedRef<IDetailCustomization> FFoliageType_InstancedStaticMeshPaintingCustomization::MakeInstance(FEdModeFoliage* InFoliageEditMode)
{
	auto Instance = MakeShareable(new FFoliageType_InstancedStaticMeshPaintingCustomization(InFoliageEditMode));
	return Instance;
}

FFoliageType_InstancedStaticMeshPaintingCustomization::FFoliageType_InstancedStaticMeshPaintingCustomization(FEdModeFoliage* InFoliageEditMode)
	: FoliageEditMode(InFoliageEditMode)
{
}

void FFoliageType_InstancedStaticMeshPaintingCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	DetailLayoutBuilder.GetObjectsBeingCustomized(CustomizedObjects);

	// If a non-asset, non-blueprint foliage type (i.e. one that is local to a single InstancedFoliageActor) is being edited, 
	// we need to restrict the possible meshes that can be assigned
	bool bCustomizingLocalFoliageType = false;
	for (TWeakObjectPtr<UObject>& Object : CustomizedObjects)
	{
		if (UFoliageType_InstancedStaticMesh* FoliageTypeISM = Cast<UFoliageType_InstancedStaticMesh>(Object.Get()))
		{
			if (!FoliageTypeISM->IsAsset() && !FoliageTypeISM->GetClass()->ClassGeneratedBy)
			{
				bCustomizingLocalFoliageType = true;
				break;
			}
		}
	}

	if (bCustomizingLocalFoliageType)
	{
		// Cache the meshes that are not available to the type being customized
		UnavailableMeshNames.Empty();
		for (auto& TypeInfo : FoliageEditMode->GetFoliageMeshList())
		{
			UFoliageType* FoliageType = TypeInfo->Settings;
			if (!FoliageType->IsAsset() && !FoliageType->GetClass()->ClassGeneratedBy)
			{
				UnavailableMeshNames.Add(FoliageType->GetSource()->GetFName());
			}
		}

		// Create a custom entry box for the mesh that filters out meshes that are already assigned to another local foliage type
		TSharedRef<IPropertyHandle> MeshPropertyHandle = DetailLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFoliageType_InstancedStaticMesh, Mesh));
		IDetailPropertyRow& PropertyRow = DetailLayoutBuilder.EditCategory("Mesh").AddProperty(MeshPropertyHandle);
		
		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		FDetailWidgetRow	Row;
		PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

		PropertyRow.CustomWidget()
		.NameContent()
		.MinDesiredWidth(Row.NameWidget.MinWidth)
		.MaxDesiredWidth(Row.NameWidget.MaxWidth)
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		.MinDesiredWidth(Row.ValueWidget.MinWidth)
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		[
			SNew(SObjectPropertyEntryBox)
			.AllowedClass(UStaticMesh::StaticClass())
			.PropertyHandle(MeshPropertyHandle)
			.ThumbnailPool(DetailLayoutBuilder.GetThumbnailPool())
			.OnShouldFilterAsset(this, &FFoliageType_InstancedStaticMeshPaintingCustomization::OnShouldFilterAsset)
		];
	}
}

bool FFoliageType_InstancedStaticMeshPaintingCustomization::OnShouldFilterAsset(const FAssetData& AssetData) const
{
	if (AssetData.GetClass() == UStaticMesh::StaticClass())
	{
		// The unavailable meshes are already referenced and therefore loaded, so we know we won't filter if GetClass() failed
		return UnavailableMeshNames.Contains(AssetData.AssetName);
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
