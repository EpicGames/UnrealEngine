// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FEdModeFoliage;
class IDetailLayoutBuilder;
struct FAssetData;

/////////////////////////////////////////////////////
// FFoliageTypePaintingCustomization
class FFoliageType_InstancedStaticMeshPaintingCustomization : public IDetailCustomization
{
public:
	// Makes a new instance of this detail layout class
	static TSharedRef<IDetailCustomization> MakeInstance(FEdModeFoliage* InFoliageEditMode);

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

private:
	FFoliageType_InstancedStaticMeshPaintingCustomization(FEdModeFoliage* InFoliageEditMode);
	bool OnShouldFilterAsset(const FAssetData& AssetData) const;

	/** Pointer to the foliage edit mode. */
	FEdModeFoliage* FoliageEditMode;

	/** The static meshes that are not available to local foliage types (because they are already assigned to a local foliage type) */
	TArray<FName> UnavailableMeshNames;
};
