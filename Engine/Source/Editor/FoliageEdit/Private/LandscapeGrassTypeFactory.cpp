// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeGrassTypeFactory.h"

#include "LandscapeGrassType.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeGrassTypeFactory)

class FFeedbackContext;
class UClass;
class UObject;

ULandscapeGrassTypeFactory::ULandscapeGrassTypeFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = ULandscapeGrassType::StaticClass();
}

UObject* ULandscapeGrassTypeFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	auto NewGrassType = NewObject<ULandscapeGrassType>(InParent, Class, Name, Flags | RF_Transactional);

	return NewGrassType;
}
