// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Factory for LandscapeGrassType assets
 */

#pragma once

#include "Factories/Factory.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "LandscapeGrassTypeFactory.generated.h"

class FFeedbackContext;
class UClass;
class UObject;

UCLASS()
class ULandscapeGrassTypeFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	// End of UFactory interface
};
