// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralFoliageComponentDetails.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "FoliageTypeObject.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "ProceduralFoliageComponent.h"
#include "ProceduralFoliageEditorLibrary.h"
#include "ProceduralFoliageSpawner.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class IPropertyHandle;

#define LOCTEXT_NAMESPACE "ProceduralFoliageComponentDetails"

TSharedRef<IDetailCustomization> FProceduralFoliageComponentDetails::MakeInstance()
{
	return MakeShareable(new FProceduralFoliageComponentDetails());

}

void FProceduralFoliageComponentDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const FName ProceduralFoliageCategoryName("ProceduralFoliage");
	IDetailCategoryBuilder& ProceduralFoliageCategory = DetailBuilder.EditCategory(ProceduralFoliageCategoryName);

	const FText ResimulateText = LOCTEXT("ResimulateButtonText", "Resimulate");
	const FText LoadUnloadedAreasText = LOCTEXT("LoadUnloadedAreasButtonText", "Load Unloaded Areas");

	TArray< TWeakObjectPtr<UObject> > ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	for (TWeakObjectPtr<UObject>& Object : ObjectsBeingCustomized)
	{
		UProceduralFoliageComponent* Component = Cast<UProceduralFoliageComponent>(Object.Get());
		if (ensure(Component))
		{
			SelectedComponents.Add(Component);
		}
	}

	TArray<TSharedRef<IPropertyHandle>> AllProperties;
	bool bSimpleProperties = true;
	bool bAdvancedProperties = false;
	// Add all properties in the category in order
	ProceduralFoliageCategory.GetDefaultProperties(AllProperties, bSimpleProperties, bAdvancedProperties);
	for (auto& Property : AllProperties)
	{
		ProceduralFoliageCategory.AddProperty(Property);
	}

	FDetailWidgetRow& NewRow = ProceduralFoliageCategory.AddCustomRow(FText::GetEmpty());

	NewRow.ValueContent()
		.MaxDesiredWidth(120.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()			
			.Padding(4.0f)
			[
				SNew(SButton)
				.OnClicked(this, &FProceduralFoliageComponentDetails::OnResimulateClicked)
				.ToolTipText(this, &FProceduralFoliageComponentDetails::GetResimulateTooltipText)
				.IsEnabled(this, &FProceduralFoliageComponentDetails::IsResimulateEnabled)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(ResimulateText)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f)
			[
				SNew(SButton)
				.OnClicked(this, &FProceduralFoliageComponentDetails::OnLoadUnloadedAreas)
				.ToolTipText(LOCTEXT("Load_UnloadedAreas", "Load unloaded areas required to simulate."))
				.IsEnabled(this, &FProceduralFoliageComponentDetails::HasUnloadedAreas)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LoadUnloadedAreasText)
				]
			]
		];
}

FReply FProceduralFoliageComponentDetails::OnResimulateClicked()
{
	TArray<UProceduralFoliageComponent*> SelectedFoliageComponents;
	SelectedFoliageComponents.Reserve(SelectedComponents.Num());
	for (TWeakObjectPtr<UProceduralFoliageComponent>& Component : SelectedComponents)
	{
		if (Component.IsValid())
		{
			SelectedFoliageComponents.Add(Component.Get());
		}
	}
	FScopedTransaction Transaction(LOCTEXT("Resimulate_Transaction", "Procedural Foliage Simulation"));
	UProceduralFoliageEditorLibrary::ResimulateProceduralFoliageComponents(SelectedFoliageComponents);

	return FReply::Handled();
}

FReply FProceduralFoliageComponentDetails::OnLoadUnloadedAreas()
{
	for(const TWeakObjectPtr<UProceduralFoliageComponent>& Component : SelectedComponents)
	{
		if(Component.IsValid())
		{
			Component->LoadSimulatedRegion();
		}
	}

	return FReply::Handled();
}

bool FProceduralFoliageComponentDetails::IsResimulateEnabled() const
{
	FText Reason;
	return IsResimulateEnabledWithReason(Reason);
}

bool FProceduralFoliageComponentDetails::IsResimulateEnabledWithReason(FText& OutReason) const
{
	bool bCanSimulate = false;

	for(const TWeakObjectPtr<UProceduralFoliageComponent>& Component : SelectedComponents)
	{
		if(Component.IsValid())
		{
			if (!Component->FoliageSpawner)
			{
				OutReason = LOCTEXT("Resimulate_Tooltip_NeedSpawner", "Cannot generate foliage: Assign a Procedural Foliage Spawner to run the procedural foliage simulation");
				return false;
			}

			if (!bCanSimulate)
			{
				for (const FFoliageTypeObject& FoliageTypeObject : Component->FoliageSpawner->GetFoliageTypes())
				{
					// Make sure at least one foliage type is ready to spawn
					if (FoliageTypeObject.HasFoliageType())
					{
						bCanSimulate = true;
						break;
					}
				}

				if (!bCanSimulate)
				{
					OutReason = LOCTEXT("Resimulate_Tooltip_EmptySpawner", "Cannot generate foliage: The assigned Procedural Foliage Spawner does not contain any foliage types to spawn.");
					return false;
				}
			}
		}
	}

	if (bCanSimulate && HasUnloadedAreas())
	{
		OutReason = LOCTEXT("Resimulate_Tooltip_UnloadedRegion", "Cannot generate foliage: The assigned Procedural Foliage Volume covers an unloaded area.");
		return false;
	}

	OutReason = LOCTEXT("Resimulate_Tooltip", "Runs the procedural foliage spawner simulation. Replaces any existing instances spawned by a previous simulation.");
	return true;
}

FText FProceduralFoliageComponentDetails::GetResimulateTooltipText() const
{
	FText TooltipText;
	IsResimulateEnabledWithReason(TooltipText);
	return TooltipText;
}

bool FProceduralFoliageComponentDetails::HasUnloadedAreas() const
{
	for(const TWeakObjectPtr<UProceduralFoliageComponent>& Component : SelectedComponents)
	{
		if(Component.IsValid())
		{
			if (!Component->IsSimulatedRegionLoaded())
			{
				return true;
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
