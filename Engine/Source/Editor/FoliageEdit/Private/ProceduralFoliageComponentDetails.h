// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailLayoutBuilder;

class FProceduralFoliageComponentDetails : public IDetailCustomization
{
public:
	virtual ~FProceduralFoliageComponentDetails()= default;

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder );
private:
	FReply OnResimulateClicked();
	FReply OnLoadUnloadedAreas();
	bool IsResimulateEnabled() const;
	bool IsResimulateEnabledWithReason(FText& OutReason) const;
	FText GetResimulateTooltipText() const;
	bool HasUnloadedAreas() const;
private:
	TArray< TWeakObjectPtr<class UProceduralFoliageComponent> > SelectedComponents;

};
