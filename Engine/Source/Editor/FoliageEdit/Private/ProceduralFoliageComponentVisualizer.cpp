// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralFoliageComponentVisualizer.h"

#include "Components/ActorComponent.h"
#include "Engine/EngineTypes.h"
#include "Math/Box.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "ProceduralFoliageComponent.h"
#include "ProceduralFoliageSpawner.h"
#include "PrimitiveDrawingUtils.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"


static const FColor& ProcTileColor = FColor::Yellow;
static const FColor& ProcTileOverlapColor = FColor::Green;

void FProceduralFoliageComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	if (const UProceduralFoliageComponent* ProcComponent = Cast<const UProceduralFoliageComponent>(Component))
	{
		if (ProcComponent->bShowDebugTiles && ProcComponent->FoliageSpawner)
		{
			const FVector TilesOrigin = ProcComponent->GetWorldPosition();

			const float TileSize = ProcComponent->FoliageSpawner->TileSize;
			const FVector TileSizeV(TileSize, TileSize, 0.f);

			const float TileOverlap = ProcComponent->TileOverlap;
			const FVector TileOverlapV(TileOverlap, TileOverlap, 0.f);
			
			FTileLayout TileLayout;
			ProcComponent->GetTileLayout(TileLayout);

			// Draw each tile
			for (int32 X = 0; X < TileLayout.NumTilesX; ++X)
			{
				for (int32 Y = 0; Y < TileLayout.NumTilesY; ++Y)
				{
					const FVector StartOffset(X*TileSize, Y*TileSize, 0.f);
					
					// Draw the tile without overlap
					const FBox BoxInner(TilesOrigin + StartOffset, TilesOrigin + StartOffset + TileSizeV);
					DrawWireBox(PDI, BoxInner, ProcTileColor, SDPG_World);

					if (TileOverlap != 0.f)
					{
						// Draw the tile expanded by the overlap amount
						const FBox BoxOuter = BoxInner + (BoxInner.Min - TileOverlapV) + (BoxInner.Max + TileOverlapV);
						DrawWireBox(PDI, BoxOuter, ProcTileOverlapColor, SDPG_World);
					}
				}
			}
			
		}
	}
}
