// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentVisualizer.h"

class FPrimitiveDrawInterface;
class FSceneView;
class UActorComponent;

class FProceduralFoliageComponentVisualizer : public FComponentVisualizer
{
public:
	//~ Begin FComponentVisualizer Interface
	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	//~ End FComponentVisualizer Interface
};
