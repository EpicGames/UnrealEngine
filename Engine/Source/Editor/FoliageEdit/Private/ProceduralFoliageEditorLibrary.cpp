// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralFoliageEditorLibrary.h"

#include "Algo/Transform.h"
#include "FoliageEdMode.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Misc/ScopedSlowTask.h"
#include "ProceduralFoliageComponent.h"
#include "ProceduralFoliageVolume.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProceduralFoliageEditorLibrary)

struct FDesiredFoliageInstance;
template <typename T> struct TObjectPtr;

#define LOCTEXT_NAMESPACE "ProceduralFoliageEditorLibrary"

UProceduralFoliageEditorLibrary::UProceduralFoliageEditorLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UProceduralFoliageEditorLibrary::ResimulateProceduralFoliageVolumes(const TArray<AProceduralFoliageVolume*>& ProceduralFoliageVolumes)
{
	TArray<UProceduralFoliageComponent*> Components;
	Components.Reserve(ProceduralFoliageVolumes.Num());
	Algo::Transform(ProceduralFoliageVolumes, Components, [](AProceduralFoliageVolume* PFV) { return PFV->ProceduralComponent; });
	ResimulateProceduralFoliageComponents(Components);
}

void UProceduralFoliageEditorLibrary::ClearProceduralFoliageVolumes(const TArray<AProceduralFoliageVolume*>& ProceduralFoliageVolumes)
{
	TArray<UProceduralFoliageComponent*> Components;
	Components.Reserve(ProceduralFoliageVolumes.Num());
	Algo::Transform(ProceduralFoliageVolumes, Components, [](AProceduralFoliageVolume* PFV) { return PFV->ProceduralComponent; });
	ClearProceduralFoliageComponents(Components);
}

void UProceduralFoliageEditorLibrary::ClearProceduralFoliageComponents(const TArray<UProceduralFoliageComponent*>& ProceduralFoliageComponents)
{
	FScopedSlowTask SlowTask(static_cast<float>(ProceduralFoliageComponents.Num()), LOCTEXT("ClearProceduralFoliageComponents", "Clearing Procedural Foliage Components"));
	SlowTask.MakeDialogDelayed(0.5f);
	for (UProceduralFoliageComponent* Component : ProceduralFoliageComponents)
	{
		SlowTask.EnterProgressFrame(1);
		if (Component)
		{
			Component->RemoveProceduralContent();
		}
	}
}

void UProceduralFoliageEditorLibrary::ResimulateProceduralFoliageComponents(const TArray<UProceduralFoliageComponent*>& ProceduralFoliageComponents)
{
	FScopedSlowTask SlowTask(static_cast<float>(ProceduralFoliageComponents.Num()), LOCTEXT("ResimulateProceduralFoliageComponents", "Resimulating Procedural Foliage Components"));
	SlowTask.MakeDialogDelayed(0.5f);
	for (UProceduralFoliageComponent* Component : ProceduralFoliageComponents)
	{
		SlowTask.EnterProgressFrame(1);
		if (Component && Component->ResimulateProceduralFoliage([Component](const TArray<FDesiredFoliageInstance>& DesiredFoliageInstances)
			{
				FFoliagePaintingGeometryFilter OverrideGeometryFilter;
				OverrideGeometryFilter.bAllowLandscape = Component->bAllowLandscape;
				OverrideGeometryFilter.bAllowStaticMesh = Component->bAllowStaticMesh;
				OverrideGeometryFilter.bAllowBSP = Component->bAllowBSP;
				OverrideGeometryFilter.bAllowFoliage = Component->bAllowFoliage;
				OverrideGeometryFilter.bAllowTranslucent = Component->bAllowTranslucent;
				FEdModeFoliage::AddInstances(Component->GetWorld(), DesiredFoliageInstances, OverrideGeometryFilter, true);
			}))
		{
			// If no instances were spawned, inform the user
			if (!Component->HasSpawnedAnyInstances())
			{
				FNotificationInfo Info(LOCTEXT("NothingSpawned_Notification", "Unable to spawn instances. Ensure a large enough surface exists within the volume."));
				Info.bUseLargeFont = false;
				Info.bFireAndForget = true;
				Info.bUseThrobber = false;
				Info.bUseSuccessFailIcons = true;
				FSlateNotificationManager::Get().AddNotification(Info);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
