// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralFoliageSpawnerFactory.h"

#include "ProceduralFoliageSpawner.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProceduralFoliageSpawnerFactory)

class FFeedbackContext;
class UClass;
class UObject;

UProceduralFoliageSpawnerFactory::UProceduralFoliageSpawnerFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UProceduralFoliageSpawner::StaticClass();
}

UObject* UProceduralFoliageSpawnerFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	auto NewProceduralFoliage = NewObject<UProceduralFoliageSpawner>(InParent, Class, Name, Flags | RF_Transactional);

	return NewProceduralFoliage;
}

bool UProceduralFoliageSpawnerFactory::ShouldShowInNewMenu() const
{
	return GetDefault<UEditorExperimentalSettings>()->bProceduralFoliage;
}
