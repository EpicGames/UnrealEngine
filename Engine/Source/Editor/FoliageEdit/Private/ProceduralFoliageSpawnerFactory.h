// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Factory for ProceduralFoliageSpawner assets
 */

#pragma once

#include "Factories/Factory.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ProceduralFoliageSpawnerFactory.generated.h"

class FFeedbackContext;
class UClass;
class UObject;

UCLASS()
class UProceduralFoliageSpawnerFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;
	// End of UFactory interface
};
