// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFoliageEdit.h"

#include "Delegates/Delegate.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "Engine/World.h"
#include "FoliageEdMode.h"
#include "FoliageEditActions.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "SFoliagePalette.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Templates/Function.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SHeader.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
class ULevel;

#define LOCTEXT_NAMESPACE "FoliageEd_Mode"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SFoliageEdit::Construct(const FArguments& InArgs)
{
	FoliageEditMode = (FEdModeFoliage*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Foliage);

	// Everything (or almost) uses this padding, change it to expand the padding.
	FMargin StandardPadding(6.f, 3.f);
	FMargin StandardLeftPadding(6.f, 3.f, 3.f, 3.f);
	FMargin StandardRightPadding(3.f, 3.f, 6.f, 3.f);

	FSlateFontInfo StandardFont = FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont"));

	const FText BlankText = FText::GetEmpty();

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 0.f, 0.f, 5.f)
		[
			SAssignNew(ErrorText, SErrorText)
		]
		+ SVerticalBox::Slot()
		.Padding(0.f)
		[
			SNew(SVerticalBox)
			.IsEnabled(this, &SFoliageEdit::IsFoliageEditorEnabled)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
				.Padding(StandardPadding)
				[
					SNew(SVerticalBox)

					// Active Tool Title
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.Padding(StandardLeftPadding)
						.HAlign(HAlign_Left)
						[
							SNew(STextBlock)
							.Text(this, &SFoliageEdit::GetActiveToolName)
							.TextStyle(FAppStyle::Get(), "FoliageEditMode.ActiveToolName.Text")
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(StandardPadding)
					[
						SNew(SHeader)
						.Visibility(this, &SFoliageEdit::GetVisibility_Options)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("OptionHeader", "Brush Options"))
							.Font(StandardFont)
						]
					]

					// Brush Size
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						.ToolTipText(LOCTEXT("BrushSize_Tooltip", "The size of the foliage brush"))
						.Visibility(this, &SFoliageEdit::GetVisibility_Radius)

						+ SHorizontalBox::Slot()
						.Padding(StandardLeftPadding)
						.FillWidth(1.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("BrushSize_Legacy", "Brush Size"))
							.Font(StandardFont)
						]
						+ SHorizontalBox::Slot()
						.Padding(StandardRightPadding)
						.FillWidth(2.0f)
						.MaxWidth(100.f)
						.VAlign(VAlign_Center)
						[
							SNew(SNumericEntryBox<float>)
							.Font(StandardFont)
							.AllowSpin(true)
							.MinValue(0.0f)
							.MaxValue(65536.0f)
							.MaxSliderValue(8192.0f)
							.MinDesiredValueWidth(50.0f)
							.SliderExponent(3.0f)
							.Value(this, &SFoliageEdit::GetRadius)
							.OnValueChanged(this, &SFoliageEdit::SetRadius)
							.IsEnabled(this, &SFoliageEdit::IsEnabled_BrushSize)
						]						
					]

					// Paint Density
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						.ToolTipText(LOCTEXT("PaintDensity_Tooltip", "The density of foliage to paint. This is a multiplier for the individual foliage type's density specifier."))
						.Visibility(this, &SFoliageEdit::GetVisibility_PaintDensity)

						+ SHorizontalBox::Slot()
						.Padding(StandardLeftPadding)
						.FillWidth(1.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PaintDensity", "Paint Density"))
							.Font(StandardFont)
						]
						+ SHorizontalBox::Slot()
						.Padding(StandardRightPadding)
						.FillWidth(2.0f)
						.MaxWidth(100.f)
						.VAlign(VAlign_Center)
						[
							SNew(SNumericEntryBox<float>)
							.Font(StandardFont)
							.AllowSpin(true)
							.MinValue(0.0f)
							.MaxValue(1.0f)
							.MaxSliderValue(1.0f)
							.MinDesiredValueWidth(50.0f)
							.Value(this, &SFoliageEdit::GetPaintDensity)
							.OnValueChanged(this, &SFoliageEdit::SetPaintDensity)
							.IsEnabled(this, &SFoliageEdit::IsEnabled_PaintDensity)
						]
					]

					// Erase Density
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						.ToolTipText(LOCTEXT("EraseDensity_Tooltip", "The density of foliage to leave behind when erasing with the Shift key held. 0 will remove all foliage."))
						.Visibility(this, &SFoliageEdit::GetVisibility_EraseDensity)

						+ SHorizontalBox::Slot()
						.Padding(StandardLeftPadding)
						.FillWidth(1.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("EraseDensity_Legacy", "Erase Density"))
							.Font(StandardFont)
						]
						+ SHorizontalBox::Slot()
						.Padding(StandardRightPadding)
						.FillWidth(2.0f)
						.MaxWidth(100.f)
						.VAlign(VAlign_Center)
						[
							SNew(SNumericEntryBox<float>)
							.Font(StandardFont)
							.AllowSpin(true)
							.MinValue(0.0f)
							.MaxValue(1.0f)
							.MaxSliderValue(1.0f)
							.MinDesiredValueWidth(50.0f)
							.Value(this, &SFoliageEdit::GetEraseDensity)
							.OnValueChanged(this, &SFoliageEdit::SetEraseDensity)
							.IsEnabled(this, &SFoliageEdit::IsEnabled_EraseDensity)
						]
					]
					
					+ SVerticalBox::Slot()
					.Padding(StandardPadding)
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						.Visibility(this, &SFoliageEdit::GetVisibility_Options)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.MaxWidth(140)
						.Padding(StandardLeftPadding)
						[
							SNew(SCheckBox)
							.Visibility(this, &SFoliageEdit::GetVisibility_SingleInstantiationMode)
							.OnCheckStateChanged_Lambda([this] (ECheckBoxState NewCheckState) { OnCheckStateChanged_SingleInstantiationMode(NewCheckState == ECheckBoxState::Checked ? true : false); } )
							.IsChecked_Lambda([this] { return SFoliageEdit::GetCheckState_SingleInstantiationMode() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
							.ToolTipText(LOCTEXT("SingleInstantiationModeTooltips", "Paint a single foliage instance at the mouse cursor location (i + Mouse Click)"))
							[
								SNew(STextBlock)
								.Text(LOCTEXT("SingleInstantiationMode", "Single Instance Mode: "))
								.Font(StandardFont)
							]
						]
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						.Padding(StandardRightPadding)
						[
							SNew(SComboButton)
							.Visibility(this, &SFoliageEdit::GetVisibility_SingleInstantiationPlacementMode)
							.IsEnabled(this, &SFoliageEdit::GetIsEnabled_SingleInstantiationPlacementMode)
							.OnGetMenuContent(this, &SFoliageEdit::GetSingleInstantiationModeMenuContent)
							.ContentPadding(2.f)
							.ToolTipText(LOCTEXT("SingleInstantiationPlacementModeToolTips", "Changes the placement mode when using single instance"))
							.ButtonContent()
							[
								SNew(STextBlock)
								.Text(this, &SFoliageEdit::GetCurrentSingleInstantiationPlacementModeText)
							]
						]
					]

					+ SVerticalBox::Slot()
					.Padding(StandardPadding)
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						.Visibility(this, &SFoliageEdit::GetVisibility_Options)

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(StandardPadding)
						[
							SNew(SWrapBox)
							.UseAllottedSize(true)
							.InnerSlotPadding({6, 5})

							+ SWrapBox::Slot()
							[
								SNew(SBox)
								.MinDesiredWidth(150.f)
								[
									SNew(SCheckBox)
									.Visibility(this, &SFoliageEdit::GetVisibility_SpawnInCurrentLevelMode)
									.OnCheckStateChanged(this, &SFoliageEdit::OnCheckStateChanged_SpawnInCurrentLevelMode)
									.IsChecked(this, &SFoliageEdit::GetCheckState_SpawnInCurrentLevelMode)
									.ToolTipText(LOCTEXT("SpawnInCurrentLevelModeTooltips", "Whether to place foliage meshes in the current level or in the level containing the mesh being painted on."))
									[
										SNew(STextBlock)
										.Text(LOCTEXT("SpawnInCurrentLevelMode", "Place in Current Level"))
										.Font(StandardFont)
									]
								]
							]
						]
					]

					// Filters
					+ SVerticalBox::Slot()
					.Padding(StandardPadding)
					.AutoHeight()
					[
						SNew(SHeader)
						.Visibility(this, &SFoliageEdit::GetVisibility_Filters)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("FiltersHeader", "Filters"))
							.Font(StandardFont)
						]
					]

					+ SVerticalBox::Slot()
					.Padding(StandardPadding)
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						.Visibility(this, &SFoliageEdit::GetVisibility_Filters)

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(StandardPadding)
						[
							SNew(SWrapBox)
							.UseAllottedSize(true)
							.InnerSlotPadding({6, 5})

							+ SWrapBox::Slot()
							[
								SNew(SBox)
								.MinDesiredWidth(91.f)
								.Visibility(this, &SFoliageEdit::GetVisibility_LandscapeFilter)
								[
									SNew(SCheckBox)
									.Visibility(this, &SFoliageEdit::GetVisibility_Filters)
									.OnCheckStateChanged(this, &SFoliageEdit::OnCheckStateChanged_Landscape)
									.IsChecked(this, &SFoliageEdit::GetCheckState_Landscape)
									.ToolTipText(this, &SFoliageEdit::GetTooltipText_Landscape)
									[
										SNew(STextBlock)
										.Text(LOCTEXT("Landscape", "Landscape"))
										.Font(StandardFont)
									]
								]
							]

							+ SWrapBox::Slot()
							[
								SNew(SBox)
								.MinDesiredWidth(91.f)
								[
									SNew(SCheckBox)
									.Visibility(this, &SFoliageEdit::GetVisibility_Filters)
									.OnCheckStateChanged(this, &SFoliageEdit::OnCheckStateChanged_StaticMesh)
									.IsChecked(this, &SFoliageEdit::GetCheckState_StaticMesh)
									.ToolTipText(this, &SFoliageEdit::GetTooltipText_StaticMesh)
									[
										SNew(STextBlock)
										.Text(LOCTEXT("StaticMeshes", "Static Meshes"))
										.Font(StandardFont)
									]
								]
							]

							+ SWrapBox::Slot()
							[
								SNew(SBox)
								.MinDesiredWidth(91.f)
								[
									SNew(SCheckBox)
									.Visibility(this, &SFoliageEdit::GetVisibility_Filters)
									.OnCheckStateChanged(this, &SFoliageEdit::OnCheckStateChanged_BSP)
									.IsChecked(this, &SFoliageEdit::GetCheckState_BSP)
									.ToolTipText(this, &SFoliageEdit::GetTooltipText_BSP)
									[
										SNew(STextBlock)
										.Text(LOCTEXT("BSP", "BSP"))
										.Font(StandardFont)
									]
								]
							]

							+ SWrapBox::Slot()
							[
								SNew(SBox)
								.MinDesiredWidth(91.f)
								[
									SNew(SCheckBox)
									.Visibility(this, &SFoliageEdit::GetVisibility_Filters)
									.OnCheckStateChanged(this, &SFoliageEdit::OnCheckStateChanged_Foliage)
									.IsChecked(this, &SFoliageEdit::GetCheckState_Foliage)
									.ToolTipText(this, &SFoliageEdit::GetTooltipText_Foliage)
									[
										SNew(STextBlock)
										.Text(LOCTEXT("Foliage", "Foliage"))
										.Font(StandardFont)
									]
								]
							]

							+ SWrapBox::Slot()
							[
								SNew(SBox)
								.MinDesiredWidth(91.f)
								[
									SNew(SCheckBox)
									.Visibility(this, &SFoliageEdit::GetVisibility_Filters)
									.OnCheckStateChanged(this, &SFoliageEdit::OnCheckStateChanged_Translucent)
									.IsChecked(this, &SFoliageEdit::GetCheckState_Translucent)
									.ToolTipText(this, &SFoliageEdit::GetTooltipText_Translucent)
									[
										SNew(STextBlock)
										.Text(LOCTEXT("Translucent", "Translucent"))
										.Font(StandardFont)
									]
								]
							]

						]
					]

				
				]
		]

		// Foliage Palette
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		.VAlign(VAlign_Fill)
		.Padding(0.f, 5.f, 0.f, 0.f)
		[
			SAssignNew(FoliagePalette, SFoliagePalette)
			.FoliageEdMode(FoliageEditMode)
		]
		]
	];

	RefreshFullList();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void SFoliageEdit::CustomizeToolBarPalette(FToolBarBuilder& ToolBarBuilder)
{
	// NOTES 
	// 
	// Legacy Foliage mode treated single instance as a setting rather than a tool.
	// The following code can be cleaned up once the Legacy Foliage Mode is removed, by clearing
	// the SingleInstantiation flag in FEdModeFoliage::ClearAllToolSelection.  Instead here,
	// we explicitly clear it in Paint, Reapply, and Fill 
	// 


	//  Select
	ToolBarBuilder.AddToolBarButton(FFoliageEditCommands::Get().SetSelect);

	//  Select All
	ToolBarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SFoliageEdit::OnSelectAllInstances)),
		NAME_None,
		LOCTEXT("FoliageSelectAll", "All"),
		LOCTEXT("FoliageSelectAllTooltip", "Select All Foliage"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "FoliageEditMode.SelectAll")
		);

	// Deselect All
	ToolBarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SFoliageEdit::OnDeselectAllInstances)),
		NAME_None,
		LOCTEXT("FoliageDeselectAll", "Deselect"),
		LOCTEXT("FoliageDeselectAllTooltip", "Deselect All Foliage Instances"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "FoliageEditMode.DeselectAll")
		);

	// Select Invalid
	ToolBarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SFoliageEdit::OnSelectInvalidInstances)),
		NAME_None,
		LOCTEXT("FoliageSelectInvalid", "Invalid"),
		LOCTEXT("FoliageSelectInvalidTooltip", "Select Invalid Foliage Instances"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "FoliageEditMode.SelectInvalid")
		);

	//  Lasso
	ToolBarBuilder.AddToolBarButton(FFoliageEditCommands::Get().SetLassoSelect);

	ToolBarBuilder.AddSeparator();

	//  Paint
	ToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateLambda( [this] { 
				FoliageEditMode->OnSetPaint();
				OnCheckStateChanged_SingleInstantiationMode(false);
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda( [this] {
				return  IsPaintTool() && !IsPlaceTool() && !IsEraseTool();
			})
		),
		NAME_None,
		LOCTEXT("FoliagePaint", "Paint"),
		LOCTEXT("FoliagePaintTooltip", "Paint the Selected Foliage"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "FoliageEditMode.SetPaint"),
		EUserInterfaceActionType::ToggleButton

	);

	//  Reapply
	ToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateLambda( [this] { 
				FoliageEditMode->OnSetReapplySettings();
				OnCheckStateChanged_SingleInstantiationMode(false);
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda( [this] {
				return !GetCheckState_SingleInstantiationMode() && IsReapplySettingsTool();
			})
		),
		NAME_None,
		LOCTEXT("FoliageReapply", "Reapply"),
		LOCTEXT("FoliageReapplyTooltip", "Reapply current settings to foliage instances"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "FoliageEditMode.SetReapplySettings"),
		EUserInterfaceActionType::ToggleButton
	);

	//  Place (Single Instance)
	ToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP( FoliageEditMode, &FEdModeFoliage::OnSetPlace ),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SFoliageEdit::IsPlaceTool)
		),
		NAME_None,
		LOCTEXT("FoliagePlace", "Single"),
		LOCTEXT("FoliagePlaceTooltip", "Place a Single Instance of the Selected Foliage"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "FoliageEditMode.Foliage"),
		EUserInterfaceActionType::ToggleButton
	);

	//  Fill
	ToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateLambda( [this] { 
				FoliageEditMode->OnSetPaintFill();
				OnCheckStateChanged_SingleInstantiationMode(false);
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SFoliageEdit::IsPaintFillTool)
		),
		NAME_None,
		LOCTEXT("FoliageFill", "Fill"),
		LOCTEXT("FoliageFillTooltip", "Fill the selected target with foliage."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "FoliageEditMode.SetPaintBucket"),
		EUserInterfaceActionType::ToggleButton
	);

	// Erase
	ToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(FoliageEditMode, &FEdModeFoliage::OnSetErase),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SFoliageEdit::IsEraseTool)
		),
		NAME_None,
		LOCTEXT("FoliageErase", "Erase"),
		LOCTEXT("FoliageEraseTooltip", "Erase the selected foliage"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "FoliageEditMode.Erase"),
		EUserInterfaceActionType::ToggleButton
	);

	ToolBarBuilder.AddSeparator();
	
	// Remove
	ToolBarBuilder.AddToolBarButton(
		FExecuteAction::CreateLambda( [this] { FoliageEditMode->RemoveSelectedInstances(FoliageEditMode->GetWorld()); } ),
		NAME_None,
		LOCTEXT("FoliageRemove", "Remove"),
		LOCTEXT("FoliageRemoveTooltip", "Remove the selected foliage"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "FoliageEditMode.Remove"),
		EUserInterfaceActionType::Button
	);

	// Move To Current Level
	ToolBarBuilder.AddToolBarButton(
		FExecuteAction::CreateSP(this, &SFoliageEdit::OnMoveSelectedInstancesToActorEditorContext),
		NAME_None,
		LOCTEXT("FoliageMoveToCurrentContext", "Move"),
		LOCTEXT("FoliageMoveToCurrentContextTooltip", "Move the Selected Foliage to the Current Context"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "FoliageEditMode.MoveToActorEditorContext")
	);

}

TSharedRef<SWidget> SFoliageEdit::MakeFilterMenu()
{
	FMenuBuilder MenuBuilder(false, nullptr);//, AddMenuExtender);

	FFoliageUISettings& UISettings = FoliageEditMode->UISettings;

	MenuBuilder.BeginSection("FoliagePlacementFilters");

	// Landscape	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("FilterLandscape", "Landscape"),
		LOCTEXT("FilterLandscapeTooltip", "Allow Foliage to be placed on Landscape"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda( [this] { FoliageEditMode->UISettings.SetFilterLandscape( !FoliageEditMode->UISettings.GetFilterLandscape()); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw( &FoliageEditMode->UISettings, &FFoliageUISettings::GetFilterLandscape)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	// StaticMesh
	MenuBuilder.AddMenuEntry(
		LOCTEXT("FilterStaticMesh", "StaticMesh"),
		LOCTEXT("FilterStaticMeshTooltip", "Allow Foliage to be placed on StaticMesh"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda( [this] { FoliageEditMode->UISettings.SetFilterStaticMesh( !FoliageEditMode->UISettings.GetFilterStaticMesh()); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw( &FoliageEditMode->UISettings, &FFoliageUISettings::GetFilterStaticMesh)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	// BSP
	MenuBuilder.AddMenuEntry(
		LOCTEXT("FilterBSP", "BSP"),
		LOCTEXT("FilterBSPTooltip", "Allow Foliage to be placed on BSP"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda( [this] { FoliageEditMode->UISettings.SetFilterBSP( !FoliageEditMode->UISettings.GetFilterBSP()); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw( &FoliageEditMode->UISettings, &FFoliageUISettings::GetFilterBSP)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	// Foliage
	MenuBuilder.AddMenuEntry(
		LOCTEXT("FilterFoliage", "Foliage"),
		LOCTEXT("FilterFoliageTooltip", "Allow Foliage to be placed on Foliage"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda( [this] { FoliageEditMode->UISettings.SetFilterFoliage( !FoliageEditMode->UISettings.GetFilterFoliage()); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw( &FoliageEditMode->UISettings, &FFoliageUISettings::GetFilterFoliage)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	// Translucent
	MenuBuilder.AddMenuEntry(
		LOCTEXT("FilterTranslucent", "Translucent"),
		LOCTEXT("FilterTranslucentTooltip", "Allow Foliage to be placed on Translucent"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda( [this] { FoliageEditMode->UISettings.SetFilterTranslucent( !FoliageEditMode->UISettings.GetFilterTranslucent()); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw( &FoliageEditMode->UISettings, &FFoliageUISettings::GetFilterTranslucent)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SFoliageEdit::MakeSettingsMenu()
{
	FMenuBuilder MenuBuilder(false, nullptr);//, AddMenuExtender);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SettingsCurrentLevel", "Place In Current Level"),
		LOCTEXT("SettingsCurrentLevelTooltip", "Allow Foliage to be placed on Translucent"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda( [this] { 
				FoliageEditMode->UISettings.SetSpawnInCurrentLevelMode( !FoliageEditMode->UISettings.GetIsInSpawnInCurrentLevelMode() ); 
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw( &FoliageEditMode->UISettings, &FFoliageUISettings::GetIsInSpawnInCurrentLevelMode)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	return MenuBuilder.MakeWidget();
}

void SFoliageEdit::RefreshFullList()
{
	FoliagePalette->UpdatePalette(true);
}

void SFoliageEdit::NotifyFoliageTypeMeshChanged(UFoliageType* FoliageType)
{
	FoliagePalette->UpdateThumbnailForType(FoliageType);
}

void SFoliageEdit::ReflectSelectionInPalette()
{
	FoliagePalette->ReflectSelectionInPalette();
}

bool SFoliageEdit::IsFoliageEditorEnabled() const
{
	ErrorText->SetError(GetFoliageEditorErrorText());

	return FoliageEditMode->IsEditingEnabled();
}

FText SFoliageEdit::GetFoliageEditorErrorText() const
{
	EFoliageEditingState EditState = FoliageEditMode->GetEditingState();

	switch (EditState)
	{
		case EFoliageEditingState::SIEWorld: return LOCTEXT("IsSimulatingError_edit", "Can't edit foliage while simulating!");
		case EFoliageEditingState::PIEWorld: return LOCTEXT("IsPIEError_edit", "Can't edit foliage in PIE!");
		case EFoliageEditingState::Enabled: return FText::GetEmpty();
		default: checkNoEntry();
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> SFoliageEdit::BuildToolBar()
{
	FVerticalToolBarBuilder Toolbar(FoliageEditMode->UICommandList, FMultiBoxCustomization::None);
	Toolbar.SetLabelVisibility(EVisibility::Collapsed);
	Toolbar.SetStyle(&FAppStyle::Get(), "FoliageEditToolbar");
	{
		Toolbar.AddToolBarButton(FFoliageEditCommands::Get().SetPaint);
		Toolbar.AddToolBarButton(FFoliageEditCommands::Get().SetReapplySettings);
		Toolbar.AddToolBarButton(FFoliageEditCommands::Get().SetSelect);
		Toolbar.AddToolBarButton(FFoliageEditCommands::Get().SetLassoSelect);
		Toolbar.AddToolBarButton(FFoliageEditCommands::Get().SetPaintBucket);
	}

	return
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SBorder)
				.HAlign(HAlign_Center)
				.Padding(0)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
				[
					Toolbar.MakeWidget()
				]
			]
		];
}

bool SFoliageEdit::IsPaintTool() const
{
	return FoliageEditMode->UISettings.GetPaintToolSelected();
}

bool SFoliageEdit::IsReapplySettingsTool() const
{
	return FoliageEditMode->UISettings.GetReapplyToolSelected();
}

bool SFoliageEdit::IsSelectTool() const
{
	return FoliageEditMode->UISettings.GetSelectToolSelected();
}

bool SFoliageEdit::IsLassoSelectTool() const
{
	return FoliageEditMode->UISettings.GetLassoSelectToolSelected();
}

bool SFoliageEdit::IsPaintFillTool() const
{
	return FoliageEditMode->UISettings.GetPaintBucketToolSelected();
}

bool SFoliageEdit::IsPlaceTool() const
{
	return IsPaintTool() && FoliageEditMode->UISettings.IsInAnySingleInstantiationMode();
}

bool SFoliageEdit::IsEraseTool() const
{
	return IsPaintTool() && FoliageEditMode->UISettings.IsInAnyEraseMode();
}

FText SFoliageEdit::GetActiveToolName() const
{
	FText OutText;
	if (IsEraseTool())
	{
		OutText = LOCTEXT("FoliageToolName_Erase", "Erase");
	}
	else if (IsPlaceTool())
	{
		OutText = LOCTEXT("FoliageToolName_Place", "Place Single Instances");
	}
	else if (IsPaintTool())
	{
		OutText = LOCTEXT("FoliageToolName_Paint", "Paint");
	}
	else if (IsReapplySettingsTool())
	{
		OutText = LOCTEXT("FoliageToolName_Reapply", "Reapply");
	}
	else if (IsSelectTool())
	{
		OutText = LOCTEXT("FoliageToolName_Select", "Select");
	}
	else if (IsLassoSelectTool())
	{
		OutText = LOCTEXT("FoliageToolName_LassoSelect", "Lasso Select");
	}
	else if (IsPaintFillTool())
	{
		OutText = LOCTEXT("FoliageToolName_Fill", "Fill");
	}

	return OutText;
}

FText SFoliageEdit::GetActiveToolMessage() const
{
	FText OutText;
	if (IsEraseTool())
	{
		OutText = LOCTEXT("FoliageToolMessage_Erase", "Click and drag in the viewport to reduce foliage down to the Erase Density Setting.");
	}
	else if (IsPlaceTool())
	{
		OutText = LOCTEXT("foliagetoolmessage_place", "Place single instances of the selected Foliage in your level.  Be sure to add foliage to your palette and ensure they are active using the checkbox on the foliage thumbnail.  Use the dropdown next to the place tool to choose between cycling placement of individual instances or placing clumps containing one of each selected instance.");
	}
	else if (IsPaintTool())
	{
		OutText = LOCTEXT("foliagetoolmessage_paint", "Click and drag directly in the viewport to paint selected foliage in your level. Be sure to add foliage to your Mesh List and ensure they are selected using the checkbox on the foliage thumbnail.");
	}
	else if (IsReapplySettingsTool())
	{
		OutText = LOCTEXT("FoliageToolMessage_Reapply", "Click and drag directly on Foliage already placed in the world to selectively change certain parameters for Foliage meshes. When you paint over Foliage meshes with the Reapply tool active, the Foliage meshes that are selected in the Mesh List will be updated to reflect the changes made in the Reapply tool.");
	}
	else if (IsSelectTool())
	{
		OutText = LOCTEXT("FoliageToolMessage_Select", "Click to select individual instances of foliage.  Use the Viewport Gizmo to adjust foliage positions.");
	}
	else if (IsLassoSelectTool())
	{
		OutText = LOCTEXT("FoliageToolMessage_LassoSelect", "Select multiple Foliage meshes simultaneously.  Use the Filter settings to control the selection of Foliage meshes that are placed on certain object types.  Once you have a selection, you can switch to the regular Select tool and do things like duplicate, move or remove Foliage meshes.");
	}
	else if (IsPaintFillTool())
	{
		OutText = LOCTEXT("FoliageToolMessage_Fill", "Click to cover objects with foliage.  You can filter what types of objects to populate using Filter in the Toolbar.");
	}
	return OutText;
}

EVisibility SFoliageEdit::GetVisibility_DataLayer() const
{
	if (UWorld::IsPartitionedWorld(FoliageEditMode->GetWorld()) && (IsPaintTool() || IsPlaceTool() || IsReapplySettingsTool() || IsPaintFillTool()))
	{
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

void SFoliageEdit::SetRadius(float InRadius)
{
	FoliageEditMode->UISettings.SetRadius(InRadius);
}

TOptional<float> SFoliageEdit::GetRadius() const
{
	return FoliageEditMode->UISettings.GetRadius();
}

bool SFoliageEdit::IsEnabled_BrushSize() const
{
	return IsLassoSelectTool() || !FoliageEditMode->UISettings.IsInAnySingleInstantiationMode();
}

void SFoliageEdit::SetPaintDensity(float InDensity)
{
	FoliageEditMode->UISettings.SetPaintDensity(InDensity);
}

TOptional<float> SFoliageEdit::GetPaintDensity() const
{
	return FoliageEditMode->UISettings.GetPaintDensity();
}

bool SFoliageEdit::IsEnabled_PaintDensity() const
{
	return !IsLassoSelectTool() && !FoliageEditMode->UISettings.IsInAnySingleInstantiationMode();
}

void SFoliageEdit::SetEraseDensity(float InDensity)
{
	FoliageEditMode->UISettings.SetUnpaintDensity(InDensity);
}

TOptional<float> SFoliageEdit::GetEraseDensity() const
{
	return FoliageEditMode->UISettings.GetUnpaintDensity();
}

bool SFoliageEdit::IsEnabled_EraseDensity() const
{
	return !IsLassoSelectTool() && !FoliageEditMode->UISettings.IsInAnySingleInstantiationMode();
}

EVisibility SFoliageEdit::GetVisibility_SelectionOptions() const
{
	return IsSelectTool() || IsLassoSelectTool() ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
}

void SFoliageEdit::ExecuteOnAllCurrentLevelFoliageTypes(TFunctionRef<void(const TArray<const UFoliageType*>&)> ExecuteFunc)
{
	TArray<FFoliageMeshUIInfoPtr>& FoliageUIList = FoliageEditMode->GetFoliageMeshList();
	TArray<const UFoliageType*> FoliageTypes;
	FoliageTypes.Reserve(FoliageUIList.Num());

	for (FFoliageMeshUIInfoPtr& TypeInfo : FoliageUIList)
	{
		if (TypeInfo->InstanceCountCurrentLevel > 0)
		{
			FoliageTypes.Add(TypeInfo->Settings);
		}
	}

	ExecuteFunc(FoliageTypes);
}

void SFoliageEdit::OnSelectAllInstances()
{
	ExecuteOnAllCurrentLevelFoliageTypes([&](const TArray<const UFoliageType*>& FoliageTypes)
	{
		FoliageEditMode->SelectInstances(FoliageTypes, true);
	});
}

void SFoliageEdit::OnSelectInvalidInstances()
{
	ExecuteOnAllCurrentLevelFoliageTypes([&](const TArray<const UFoliageType*>& FoliageTypes)
	{
		FoliageEditMode->SelectInstances(FoliageTypes, false);
		FoliageEditMode->SelectInvalidInstances(FoliageTypes);
	});
}

void SFoliageEdit::OnDeselectAllInstances()
{
	ExecuteOnAllCurrentLevelFoliageTypes([&](const TArray<const UFoliageType*>& FoliageTypes)
	{
		FoliageEditMode->SelectInstances(FoliageTypes, false);
	});
}

void SFoliageEdit::OnMoveSelectedInstancesToActorEditorContext()
{
	if (UWorld* World = FoliageEditMode->GetWorld())
	{
		if (!World->IsPartitionedWorld())
		{
			if (ULevel* CurrentLevel = World->GetCurrentLevel())
			{
				FoliageEditMode->MoveSelectedFoliageToLevel(CurrentLevel);
			}
		}
		else
		{
			FoliageEditMode->MoveSelectedFoliageToActorEditorContext();
		}
	}
}

FText SFoliageEdit::GetFilterText() const
{
	FText TooltipText;
	if (IsPaintTool() || IsPaintFillTool())
	{
		TooltipText = LOCTEXT("PlacementFilter", "Placement Filter");
	}
	else if (IsReapplySettingsTool())
	{
		TooltipText = LOCTEXT("ReapplyFilter", "Reapply Filter");
	}
	else if (IsLassoSelectTool())
	{
		TooltipText = LOCTEXT("SelectionFilter", "Selection Filter");
	}

	return TooltipText;
}

void SFoliageEdit::OnCheckStateChanged_Landscape(ECheckBoxState InState)
{
	FoliageEditMode->UISettings.SetFilterLandscape(InState == ECheckBoxState::Checked ? true : false);
}

ECheckBoxState SFoliageEdit::GetCheckState_Landscape() const
{
	return FoliageEditMode->UISettings.GetFilterLandscape() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SFoliageEdit::OnCheckStateChanged_SingleInstantiationMode(bool InState)
{
	FoliageEditMode->UISettings.SetIsInSingleInstantiationMode(InState);
}

bool SFoliageEdit::GetCheckState_SingleInstantiationMode() const
{
	return FoliageEditMode->UISettings.GetIsInSingleInstantiationMode() || FoliageEditMode->UISettings.GetIsInQuickSingleInstantiationMode();
}

void SFoliageEdit::OnCheckStateChanged_SpawnInCurrentLevelMode(ECheckBoxState InState)
{
	FoliageEditMode->UISettings.SetSpawnInCurrentLevelMode(InState == ECheckBoxState::Checked ? true : false);
}

ECheckBoxState SFoliageEdit::GetCheckState_SpawnInCurrentLevelMode() const
{
	return FoliageEditMode->UISettings.GetIsInSpawnInCurrentLevelMode() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FText SFoliageEdit::GetTooltipText_Landscape() const
{
	FText TooltipText;
	if (IsPaintTool() || IsPaintFillTool())
	{
		TooltipText = LOCTEXT("FilterLandscapeTooltip_Placement", "Place foliage on landscapes");
	}
	else if (IsReapplySettingsTool())
	{
		TooltipText = LOCTEXT("FilterLandscapeTooltip_Reapply", "Reapply to instances on landscapes");
	}
	else if (IsLassoSelectTool())
	{
		TooltipText = LOCTEXT("FilterLandscapeTooltip_Select", "Select instances on landscapes");
	}

	return TooltipText;
}

void SFoliageEdit::OnCheckStateChanged_StaticMesh(ECheckBoxState InState)
{
	FoliageEditMode->UISettings.SetFilterStaticMesh(InState == ECheckBoxState::Checked ? true : false);
}

ECheckBoxState SFoliageEdit::GetCheckState_StaticMesh() const
{
	return FoliageEditMode->UISettings.GetFilterStaticMesh() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FText SFoliageEdit::GetTooltipText_StaticMesh() const
{
	FText TooltipText;
	if (IsPaintTool() || IsPaintFillTool())
	{
		TooltipText = LOCTEXT("FilterStaticMeshTooltip_Placement", "Place foliage on static meshes");
	}
	else if (IsReapplySettingsTool())
	{
		TooltipText = LOCTEXT("FilterStaticMeshTooltip_Reapply", "Reapply to instances on static meshes");
	}
	else if (IsLassoSelectTool())
	{
		TooltipText = LOCTEXT("FilterStaticMeshTooltip_Select", "Select instances on static meshes");
	}

	return TooltipText;
}

void SFoliageEdit::OnCheckStateChanged_BSP(ECheckBoxState InState)
{
	FoliageEditMode->UISettings.SetFilterBSP(InState == ECheckBoxState::Checked ? true : false);
}

ECheckBoxState SFoliageEdit::GetCheckState_BSP() const
{
	return FoliageEditMode->UISettings.GetFilterBSP() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FText SFoliageEdit::GetTooltipText_BSP() const
{
	FText TooltipText;
	if (IsPaintTool() || IsPaintFillTool())
	{
		TooltipText = LOCTEXT("FilterBSPTooltip_Placement", "Place foliage on BSP");
	}
	else if (IsReapplySettingsTool())
	{
		TooltipText = LOCTEXT("FilterBSPTooltip_Reapply", "Reapply to instances on BSP");
	}
	else if (IsLassoSelectTool())
	{
		TooltipText = LOCTEXT("FilterBSPTooltip_Select", "Select instances on BSP");
	}

	return TooltipText;
}

void SFoliageEdit::OnCheckStateChanged_Foliage(ECheckBoxState InState)
{
	FoliageEditMode->UISettings.SetFilterFoliage(InState == ECheckBoxState::Checked ? true : false);
}

ECheckBoxState SFoliageEdit::GetCheckState_Foliage() const
{
	return FoliageEditMode->UISettings.GetFilterFoliage() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FText SFoliageEdit::GetTooltipText_Foliage() const
{
	FText TooltipText;
	if (IsPaintTool() || IsPaintFillTool())
	{
		TooltipText = LOCTEXT("FilterFoliageTooltip_Placement", "Place foliage on other blocking foliage geometry");
	}
	else if (IsReapplySettingsTool())
	{
		TooltipText = LOCTEXT("FilterFoliageTooltip_Reapply", "Reapply to instances on blocking foliage geometry");
	}
	else if (IsLassoSelectTool())
	{
		TooltipText = LOCTEXT("FilterFoliageTooltip_Select", "Select instances on blocking foliage geometry");
	}

	return TooltipText;
}

void SFoliageEdit::OnCheckStateChanged_Translucent(ECheckBoxState InState)
{
	FoliageEditMode->UISettings.SetFilterTranslucent(InState == ECheckBoxState::Checked ? true : false);
}

ECheckBoxState SFoliageEdit::GetCheckState_Translucent() const
{
	return FoliageEditMode->UISettings.GetFilterTranslucent() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FText SFoliageEdit::GetTooltipText_Translucent() const
{
	FText TooltipText;
	if (IsPaintTool() || IsPaintFillTool())
	{
		TooltipText = LOCTEXT("FilterTranslucentTooltip_Placement", "Place foliage on translucent geometry");
	}
	else if (IsReapplySettingsTool())
	{
		TooltipText = LOCTEXT("FilterTranslucentTooltip_Reapply", "Reapply to instances on translucent geometry");
	}
	else if (IsLassoSelectTool())
	{
		TooltipText = LOCTEXT("FilterTranslucentTooltip_Select", "Select instances on translucent geometry");
	}

	return TooltipText;
}

EVisibility SFoliageEdit::GetVisibility_Radius() const
{
	if (FoliageEditMode->UISettings.GetSelectToolSelected() || FoliageEditMode->UISettings.GetReapplyPaintBucketToolSelected() || FoliageEditMode->UISettings.GetPaintBucketToolSelected() )
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

EVisibility SFoliageEdit::GetVisibility_PaintDensity() const
{
	if (!FoliageEditMode->UISettings.GetPaintToolSelected())
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

EVisibility SFoliageEdit::GetVisibility_EraseDensity() const
{
	if (!FoliageEditMode->UISettings.GetPaintToolSelected())
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

EVisibility SFoliageEdit::GetVisibility_Filters() const
{
	if (FoliageEditMode->UISettings.GetSelectToolSelected())
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

EVisibility SFoliageEdit::GetVisibility_LandscapeFilter() const
{
	// Fill tool doesn't support Landscape
	if (FoliageEditMode->UISettings.GetPaintBucketToolSelected())
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

EVisibility SFoliageEdit::GetVisibility_Actions() const
{
	if (FoliageEditMode->UISettings.GetSelectToolSelected() || FoliageEditMode->UISettings.GetLassoSelectToolSelected())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility SFoliageEdit::GetVisibility_SingleInstantiationMode() const
{
	if (FoliageEditMode->UISettings.GetPaintToolSelected() || FoliageEditMode->UISettings.GetReapplyToolSelected() || FoliageEditMode->UISettings.GetLassoSelectToolSelected())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility SFoliageEdit::GetVisibility_SingleInstantiationPlacementMode() const
{
	if (FoliageEditMode->UISettings.GetPaintToolSelected())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

bool SFoliageEdit::GetIsEnabled_SingleInstantiationPlacementMode() const
{
	return FoliageEditMode->UISettings.IsInAnySingleInstantiationMode();
}

EVisibility SFoliageEdit::GetVisibility_SpawnInCurrentLevelMode() const
{
	if (FoliageEditMode->UISettings.GetPaintToolSelected())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility SFoliageEdit::GetVisibility_Options() const
{
	if (FoliageEditMode->UISettings.GetSelectToolSelected() || FoliageEditMode->UISettings.GetPaintBucketToolSelected())
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

void SFoliageEdit::OnSingleInstantiationPlacementModeChanged(int32 InMode)
{
	FoliageEditMode->UISettings.SetSingleInstantiationPlacementMode(EFoliageSingleInstantiationPlacementMode::Type(InMode));
}

TSharedRef<SWidget> SFoliageEdit::GetSingleInstantiationModeMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	for (int32 i = 0; i < (int32)EFoliageSingleInstantiationPlacementMode::Type::ModeCount; i++)
	{
		MenuBuilder.AddMenuEntry(
			GetSingleInstantiationPlacementModeText(EFoliageSingleInstantiationPlacementMode::Type(i)), 
			FText(), 
			FSlateIcon(), 
			FUIAction(
				FExecuteAction::CreateSP(this, &SFoliageEdit::OnSingleInstantiationPlacementModeChanged, i),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, i] { return FoliageEditMode->UISettings.GetSingleInstantiationPlacementMode() == EFoliageSingleInstantiationPlacementMode::Type(i); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	return MenuBuilder.MakeWidget();
}

FText SFoliageEdit::GetSingleInstantiationPlacementModeText(EFoliageSingleInstantiationPlacementMode::Type InMode) const
{
	switch (InMode)
	{
	case EFoliageSingleInstantiationPlacementMode::Type::All:
		return LOCTEXT("SingleInstantiationPlacementModeAll", "All Selected");
	case EFoliageSingleInstantiationPlacementMode::Type::CycleThrough:
		return LOCTEXT("SingleInstantiationPlacementModeCycleThrough", "Cycle Through Selected");
	}

	return LOCTEXT("SingleInstantiationPlacementModeNone", "Invalid");
}

FText SFoliageEdit::GetCurrentSingleInstantiationPlacementModeText() const
{
	return GetSingleInstantiationPlacementModeText(FoliageEditMode->UISettings.GetSingleInstantiationPlacementMode());
}

#undef LOCTEXT_NAMESPACE
