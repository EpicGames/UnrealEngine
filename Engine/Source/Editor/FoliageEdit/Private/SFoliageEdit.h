// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 
#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FEdModeFoliage;
class FToolBarBuilder;
class SWidget;
class UFoliageType;
struct FFoliageMeshUIInfo;
template <typename FuncType> class TFunctionRef;

enum class ECheckBoxState : uint8;
namespace EFoliageSingleInstantiationPlacementMode {
	enum class Type;
};

typedef TSharedPtr<FFoliageMeshUIInfo> FFoliageMeshUIInfoPtr; //should match typedef in FoliageEdMode.h

class SFoliageEdit : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFoliageEdit) {}
	SLATE_END_ARGS()

public:
	/** SCompoundWidget functions */
	void Construct(const FArguments& InArgs);

	/** Does a full refresh on the list. */
	void RefreshFullList();

	/** Notifies the widget that the mesh assigned to a foliage type in the list has changed */
	void NotifyFoliageTypeMeshChanged(UFoliageType* FoliageType);

	/** Notifies the widget to reflect its selected foliage types based on the selected foliage instances */
	void ReflectSelectionInPalette();

	/** Gets FoliageEditMode. Used by the cluster details to notify changes */
	class FEdModeFoliage* GetFoliageEditMode() const { return FoliageEditMode; }

	/** Will return the status of editing mode */
	bool IsFoliageEditorEnabled() const;
	
	/** Get the error message for this editing mode */
	FText GetFoliageEditorErrorText() const;

	/** Modes Panel Header Information **/
	void CustomizeToolBarPalette(FToolBarBuilder& ToolBarBuilder);
	FText GetActiveToolName() const;
	FText GetActiveToolMessage() const;


private:
	/** Creates the toolbar. */
	TSharedRef<SWidget> BuildToolBar();

	/** Checks if the tool mode is Paint. */
	bool IsPaintTool() const;

	/** Checks if the tool mode is Reapply Settings. */
	bool IsReapplySettingsTool() const;

	/** Checks if the tool mode is Select. */
	bool IsSelectTool() const;

	bool IsLassoSelectTool() const;

	/** Checks if the tool mode is Paint Bucket. */
	bool IsPaintFillTool() const;

	/** Checks if the tool mode is Erase */
	bool IsEraseTool() const;

	/** Checks if the tool mode is Place Single Instance */
	bool IsPlaceTool() const;


public:	// BRUSH SETTINGS
	/** Sets the brush Radius for the brush. */
	void SetRadius(float InRadius);

	/** Retrieves the brush Radius for the brush. */
	// TOptional<float> GetRadius() const;
	TOptional<float> GetRadius() const;

	/** Checks if the brush size should appear. Dependant on the current tool being used. */
	bool IsEnabled_BrushSize() const;

	/** Sets the Paint Density for the brush. */
	void SetPaintDensity(float InDensity);

	/** Retrieves the Paint Density for the brush. */
	TOptional<float> GetPaintDensity() const;

	/** Checks if the paint density should appear. Dependant on the current tool being used. */
	bool IsEnabled_PaintDensity() const;

	/** Sets the Erase Density for the brush. */
	void SetEraseDensity(float InDensity);

	/** Retrieves the Erase Density for the brush. */
	TOptional<float> GetEraseDensity() const;

	/** Checks if the erase density should appear. Dependant on the current tool being used. */
	bool IsEnabled_EraseDensity() const;

	/** Retrieves the text for the filters option */
	FText GetFilterText() const;

	/** Create a menu for the filter options */
	TSharedRef<SWidget> MakeFilterMenu();

	/** Create a menu for the settings option */
	TSharedRef<SWidget> MakeSettingsMenu();

	/** Sets the filter settings for if painting will occur on Landscapes. */
	void OnCheckStateChanged_Landscape(ECheckBoxState InState);

	/** Retrieves the filter settings for painting on Landscapes. */
	ECheckBoxState GetCheckState_Landscape() const;

	/** Sets the instantiation mode settings */
	void OnCheckStateChanged_SingleInstantiationMode(bool InState);

	/** Retrieves the instantiation mode settings */
	bool GetCheckState_SingleInstantiationMode() const;

	/** Sets the instantiation mode settings */
	void OnCheckStateChanged_SpawnInCurrentLevelMode(ECheckBoxState InState);

	/** Retrieves the instantiation mode settings */
	ECheckBoxState GetCheckState_SpawnInCurrentLevelMode() const;

	/** Retrieves the tooltip text for the landscape filter */
	FText GetTooltipText_Landscape() const;

	/** Sets the filter settings for if painting will occur on Static Meshes. */
	void OnCheckStateChanged_StaticMesh(ECheckBoxState InState);

	/** Retrieves the filter settings for painting on Static Meshes. */
	ECheckBoxState GetCheckState_StaticMesh() const;

	/** Retrieves the tooltip text for the static mesh filter */
	FText GetTooltipText_StaticMesh() const;

	/** Sets the filter settings for if painting will occur on BSPs. */
	void OnCheckStateChanged_BSP(ECheckBoxState InState);

	/** Retrieves the filter settings for painting on BSPs. */
	ECheckBoxState GetCheckState_BSP() const;

	/** Retrieves the tooltip text for the BSP filter */
	FText GetTooltipText_BSP() const;

	/** Sets the filter settings for if painting will occur on foliage meshes. */
	void OnCheckStateChanged_Foliage(ECheckBoxState InState);

	/** Retrieves the filter settings for painting on foliage meshes. */
	ECheckBoxState GetCheckState_Foliage() const;

	/** Retrieves the tooltip text for the foliage meshes filter */
	FText GetTooltipText_Foliage() const;

	/** Sets the filter settings for if painting will occur on translucent meshes. */
	void OnCheckStateChanged_Translucent(ECheckBoxState InState);

	/** Retrieves the filter settings for painting on translucent meshes. */
	ECheckBoxState GetCheckState_Translucent() const;

	/** Retrieves the tooltip text for the translucent filter */
	FText GetTooltipText_Translucent() const;

	/** Checks if the Data Layer should appear. */
	EVisibility GetVisibility_DataLayer() const;

	/** Checks if the radius spinbox should appear. Dependant on the current tool being used. */
	EVisibility GetVisibility_Radius() const;

	/** Checks if the paint density spinbox should appear. Dependant on the current tool being used. */
	EVisibility GetVisibility_PaintDensity() const;

	/** Checks if the erase density spinbox should appear. Dependant on the current tool being used. */
	EVisibility GetVisibility_EraseDensity() const;

	/** Checks if the filters should appear. Dependant on the current tool being used. */
	EVisibility GetVisibility_Filters() const;

	/** Checks if the Landscape filter should appear. Hidden for the Fill tool as it's not supported there. */
	EVisibility GetVisibility_LandscapeFilter() const;
	
	/** Checks if the actions should appear. Dependant on the current tool being used. */
	EVisibility GetVisibility_Actions() const;

	/** Checks if the text in the empty list overlay should appear. If the list is has items but the the drag and drop override is true, it will return EVisibility::Visible. */
	EVisibility GetVisibility_EmptyList() const;

	/** @return Whether selecting instances is currently possible */
	EVisibility GetVisibility_SelectionOptions() const;

	/** Checks if the Options section should be displayed. Dependant on the current tool being used. */
	EVisibility GetVisibility_Options() const;

	/** Checks if the SingleInstantiationMode checkbox should appear. Dependant on the current tool being used. */
	EVisibility GetVisibility_SingleInstantiationMode() const;	

	/** Checks if the SingleInstantiationPlacement mode combobutton should appear. Dependant on the current tool being used. */
	EVisibility GetVisibility_SingleInstantiationPlacementMode() const;

	/** Checks if the SingleInstantiationPlacement mode combobutton should be enabled. Dependant on if single instantiation mode is enabled. */
	bool GetIsEnabled_SingleInstantiationPlacementMode() const;

	/** Checks if the SpawnInCurrentLevelMode checkbox should appear. Dependant on the current tool being used. */
	EVisibility GetVisibility_SpawnInCurrentLevelMode() const;
		
	/** Sets the single instantiation placement mode */
	void OnSingleInstantiationPlacementModeChanged(int32 InMode);

	/** @return display text for the placement type */
	FText GetSingleInstantiationPlacementModeText(EFoliageSingleInstantiationPlacementMode::Type InMode) const;

	/** @return menu of all single instantiation modes */
	TSharedRef<SWidget> GetSingleInstantiationModeMenuContent();

	/** @return display text for current placement type */
	FText GetCurrentSingleInstantiationPlacementModeText() const;

public:	// SELECTION

	/** Handler for 'Select All' command  */
	void OnSelectAllInstances();

	/** Handler for 'Select Invalid Instances' command  */
	void OnSelectInvalidInstances();

	/** Handler for 'Deselect All' command  */
	void OnDeselectAllInstances();

	/** Handler for 'Move to Current Editor Context' command*/
	void OnMoveSelectedInstancesToActorEditorContext();

	/** Tooltip text for 'Instance Count" column */
	FText GetTotalInstanceCountTooltipText() const;

	/** Handler to trigger a refresh of the details view when the active tool changes */
	void HandleOnToolChanged();

	void ExecuteOnAllCurrentLevelFoliageTypes(TFunctionRef<void(const TArray<const UFoliageType*>&)> ExecuteFunc);

private:
	/** Palette of available foliage types */
	TSharedPtr<class SFoliagePalette> FoliagePalette;
	
	/** Current error message */	
	TSharedPtr<class SErrorText> ErrorText;
		
	/** Pointer to the foliage edit mode. */
	FEdModeFoliage*					FoliageEditMode;
};
