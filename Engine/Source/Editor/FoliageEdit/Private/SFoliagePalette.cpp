// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFoliagePalette.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetSelection.h"
#include "AssetThumbnail.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "DetailsViewArgs.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Engine/StaticMesh.h"
#include "InstancedFoliageActor.h"
#include "FoliageEdMode.h"
#include "FoliagePaletteCommands.h"
#include "FoliagePaletteItem.h"
#include "FoliageType.h"
#include "FoliageTypePaintingCustomization.h"
#include "FoliageType_Actor.h"
#include "FoliageType_InstancedStaticMesh.h"
#include "FoliageType_InstancedStaticMeshPaintingCustomization.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/PlatformMisc.h"
#include "IContentBrowserSingleton.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/FeedbackContext.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "SPositiveActionButton.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateStructs.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UnrealNames.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Input/DragAndDrop.h"
#include "DragAndDrop/ExternalContentDragDropOp.h"

class UObject;
struct FGeometry;
struct FKeyEvent;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "FoliageEd_Mode"

////////////////////////////////////////////////
// SFoliageDragDropHandler
////////////////////////////////////////////////

/** Drag-drop zone for adding foliage types to the palette */
class SFoliageDragDropHandler : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFoliageDragDropHandler) {}
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_EVENT(FOnDrop, OnDrop)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		bIsDragOn = false;
		OnDropDelegate = InArgs._OnDrop;

		this->ChildSlot
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
				.BorderBackgroundColor(this, &SFoliageDragDropHandler::GetBackgroundColor)
				.Padding(FMargin(30))
				[
					InArgs._Content.Widget
				]
			];
	}

	FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		bIsDragOn = false;
		if (OnDropDelegate.IsBound())
		{
			return OnDropDelegate.Execute(MyGeometry, DragDropEvent);
		}
		
		return FReply::Handled();
	}

	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		bIsDragOn = true;
	}

	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override
	{
		bIsDragOn = false;
	}

private:
	FSlateColor GetBackgroundColor() const
	{
		return bIsDragOn ? FLinearColor(1.0f, 0.6f, 0.1f, 0.9f) : FLinearColor(0.1f, 0.1f, 0.1f, 0.9f);
	}

private:
	FOnDrop OnDropDelegate;
	bool bIsDragOn;
};

////////////////////////////////////////////////
// SUneditableFoliageTypeWarning
////////////////////////////////////////////////
class SUneditableFoliageTypeWarning : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SUneditableFoliageTypeWarning)
		: _WarningText()
		, _OnHyperlinkClicked()
	{}

	/** The rich text to show in the warning */
	SLATE_ATTRIBUTE(FText, WarningText)

		/** Called when the hyperlink in the rich text is clicked */
		SLATE_EVENT(FSlateHyperlinkRun::FOnClick, OnHyperlinkClicked)

		SLATE_END_ARGS()

		/** Constructs the widget */
		void Construct(const FArguments& InArgs)
	{
		ChildSlot
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(2)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
					]
					+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(2)
						[
							SNew(SRichTextBlock)
							.DecoratorStyleSet(&FAppStyle::Get())
							.Justification(ETextJustify::Left)
							.TextStyle(FAppStyle::Get(), "DetailsView.BPMessageTextStyle")
							.Text(InArgs._WarningText)
							.AutoWrapText(true)
							+ SRichTextBlock::HyperlinkDecorator(TEXT("HyperlinkDecorator"), InArgs._OnHyperlinkClicked)
						]
				]
			];
	}
};

////////////////////////////////////////////////
// SFoliagePalette
////////////////////////////////////////////////
void SFoliagePalette::Construct(const FArguments& InArgs)
{
	bItemsNeedRebuild = false;
	bIsUneditableFoliageTypeSelected = false;
	bIsRebuildTimerRegistered = false;

	FoliageEditMode = InArgs._FoliageEdMode;

	FoliageEditMode->OnToolChanged.AddSP(this, &SFoliagePalette::HandleOnToolChanged);

	FEditorDelegates::OnExternalContentResolved.AddSP(this, &SFoliagePalette::OnExternalContentResolved);

	FFoliagePaletteCommands::Register();
	UICommandList = MakeShareable(new FUICommandList);
	BindCommands();

	// Size of the thumbnail pool should be large enough to show a reasonable amount of foliage assets on screen at once,
	// otherwise some thumbnail images will appear duplicated.
	ThumbnailPool = MakeShared<FAssetThumbnailPool>(64);

	TypeFilter = MakeShareable(new FoliageTypeTextFilter(
		FoliageTypeTextFilter::FItemToStringArray::CreateSP(this, &SFoliagePalette::GetPaletteItemFilterString)));

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs Args;
	Args.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	Args.bAllowSearch = false; 
	Args.bHideSelectionTip = true;
	DetailsWidget = PropertyModule.CreateDetailView(Args);
	DetailsWidget->SetVisibility(FoliageEditMode->UISettings.GetShowPaletteItemDetails() ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed);
	DetailsWidget->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &SFoliagePalette::GetIsPropertyEditingEnabled));

	// We want to use our own customization for UFoliageType
	DetailsWidget->RegisterInstancedCustomPropertyLayout(UFoliageType_InstancedStaticMesh::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FFoliageType_InstancedStaticMeshPaintingCustomization::MakeInstance, FoliageEditMode)
		);
	DetailsWidget->RegisterInstancedCustomPropertyLayout(UFoliageType::StaticClass(), 
		FOnGetDetailCustomizationInstance::CreateStatic(&FFoliageTypePaintingCustomization::MakeInstance, FoliageEditMode)
		);

	const FText BlankText = FText::GetEmpty();

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(FMargin(2.f, 8.f))
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				// +Add Foliage Type button
				SAssignNew(AddFoliageTypeCombo, SPositiveActionButton)
				.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
				.Text(LOCTEXT("AddFoliageTypeButtonLabel", "Foliage"))
				.OnGetMenuContent(this, &SFoliagePalette::GetAddFoliageTypePicker)
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(6.f, 0.f)
			[
				SAssignNew(SearchBoxPtr, SSearchBox)
				.HintText(LOCTEXT("SearchFoliagePaletteHint", "Search Foliage"))
				.OnTextChanged(this, &SFoliagePalette::OnSearchTextChanged)
			]

			// Show Details  
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.ToolTipText(this, &SFoliagePalette::GetShowHideDetailsTooltipText)
				.Style(FAppStyle::Get(), "ToggleButtonCheckBox")
				.IsChecked_Lambda([this]() -> ECheckBoxState {
						return FoliageEditMode->UISettings.GetShowPaletteItemDetails() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})

				.OnCheckStateChanged(this, &SFoliagePalette::OnShowHideDetailsClicked)
				.Content()
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush("LevelEditor.Tabs.Details"))
				]
			]

			// View Options
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew( SComboButton )
				.ComboButtonStyle( FAppStyle::Get(), "SimpleComboButton" ) // Use the tool bar item style for this button
				.OnGetMenuContent(this, &SFoliagePalette::GetViewOptionsMenuContent)
				.HasDownArrow(false)
				.ButtonContent()
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image( FAppStyle::Get().GetBrush("Icons.Settings") )
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.MaxHeight(230.0f)
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(6.f, 3.f))
				[
					SNew(SBox)
					.Visibility(this, &SFoliagePalette::GetDropFoliageHintVisibility)
					.Padding(FMargin(15, 0))
					.MinDesiredHeight(30.f)
					[
						SNew(SScaleBox)
						.Stretch(EStretch::ScaleToFit)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("Foliage_DropStatic", "+ Drop Foliage Here"))
							.ToolTipText(LOCTEXT("Foliage_DropStatic_ToolTip", "Drag and drop foliage types or static meshes from the Content Browser to add them to the palette"))
						]
					]
				]

				+ SVerticalBox::Slot()
				[
					CreatePaletteViews()
				]
				
				+ SVerticalBox::Slot()
				.Padding(FMargin(0.f))
				.VAlign(VAlign_Bottom)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
						
					// Selected type name area
					+ SHorizontalBox::Slot()
					.Padding(FMargin(3.f))
					.VAlign(VAlign_Bottom)
					//.AutoWidth()
					[
						SNew(STextBlock)
						.Text(this, &SFoliagePalette::GetDetailsNameAreaText)
					]
				]
			]
				
			// Foliage Mesh Drop Zone
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SFoliageDragDropHandler)
				.Visibility(this, &SFoliagePalette::GetFoliageDropTargetVisibility)
				.OnDrop(this, &SFoliagePalette::HandleFoliageDropped)
				[
					SNew(SScaleBox)
					.Stretch(EStretch::ScaleToFit)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Foliage_AddFoliageMesh", "+ Foliage Type"))
						.ShadowOffset(FVector2D(1.f, 1.f))
					]
				]
			]
		]
		// Details
		+ SVerticalBox::Slot()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.Padding(0, 2)
			.AutoHeight()
			[
				SNew(SUneditableFoliageTypeWarning)
				.WarningText(LOCTEXT("CannotEditBlueprintFoliageTypeWarning", "Blueprint foliage types must be edited in the <a id=\"HyperlinkDecorator\" style=\"DetailsView.BPMessageHyperlinkStyle\">Blueprint</>"))
				.OnHyperlinkClicked(this, &SFoliagePalette::OnEditFoliageTypeBlueprintHyperlinkClicked)
				.Visibility(this, &SFoliagePalette::GetUneditableFoliageTypeWarningVisibility)
			]

			+ SVerticalBox::Slot()
			[
				DetailsWidget.ToSharedRef()
			]
		]
	];

	UpdatePalette(true);
}

SFoliagePalette::~SFoliagePalette()
{
}

FReply SFoliagePalette::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (UICommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

void SFoliagePalette::UpdatePalette(bool bRebuildItems)
{
	bItemsNeedRebuild |= bRebuildItems;

	if (!bIsRebuildTimerRegistered)
	{
		bIsRebuildTimerRegistered = true;
		RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SFoliagePalette::UpdatePaletteItems));

		// widget view needs to be refreshed immediately since it's possibly invalid and can't participate in layouting
		RefreshActivePaletteViewWidget();
	}
}

void SFoliagePalette::UpdateThumbnailForType(UFoliageType* FoliageType)
{
	// Recreate the palette item for the given foliage type
	for (auto& Item : PaletteItems)
	{
		if (Item->GetFoliageType() == FoliageType)
		{
			const bool bItemIsSelected = GetActiveViewWidget()->IsItemSelected(Item);

			Item = MakeShared<FFoliagePaletteItemModel>(Item->GetTypeUIInfo(), SharedThis(this), ThumbnailPool, FoliageEditMode);
			if (bItemIsSelected)
			{
				GetActiveViewWidget()->SetItemSelection(Item, true);
			}

			// If a local foliage type changed its mesh, we need to rebuild the palette to ensure a consistent order
			const bool bRebuild = !Item->IsBlueprint() && !Item->IsAsset();
			UpdatePalette(bRebuild);
			break;
		}
	}
}

bool SFoliagePalette::AnySelectedTileHovered() const
{
	for (FFoliagePaletteItemModelPtr& PaletteItem : GetActiveViewWidget()->GetSelectedItems())
	{
		TSharedPtr<ITableRow> Tile = TileViewWidget->WidgetFromItem(PaletteItem);
		if (Tile.IsValid() && Tile->AsWidget()->IsHovered())
		{
			return true;
		}
	}

	return false;
}

void SFoliagePalette::ActivateAllSelectedTypes(bool bActivate) const
{
	// Apply the new check state to all of the selected types
	for (FFoliagePaletteItemModelPtr& PaletteItem : GetActiveViewWidget()->GetSelectedItems())
	{
		PaletteItem->SetTypeActiveInPalette(bActivate);
	}
}

void SFoliagePalette::BindCommands()
{
	const FFoliagePaletteCommands& Commands = FFoliagePaletteCommands::Get();

	// Context menu commands
	UICommandList->MapAction(
		Commands.ActivateFoliageType,
		FExecuteAction::CreateSP(this, &SFoliagePalette::OnActivateFoliageTypes),
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SFoliagePalette::OnCanActivateFoliageTypes));

	UICommandList->MapAction(
		Commands.DeactivateFoliageType,
		FExecuteAction::CreateSP(this, &SFoliagePalette::OnDeactivateFoliageTypes),
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SFoliagePalette::OnCanDeactivateFoliageTypes));

	UICommandList->MapAction(
		Commands.RemoveFoliageType,
		FExecuteAction::CreateSP(this, &SFoliagePalette::OnRemoveFoliageType));

	UICommandList->MapAction(
		Commands.ShowFoliageTypeInCB,
		FExecuteAction::CreateSP(this, &SFoliagePalette::OnShowFoliageTypeInCB));

	UICommandList->MapAction(
		Commands.SelectAllInstances,
		FExecuteAction::CreateSP(this, &SFoliagePalette::OnSelectAllInstances),
		FCanExecuteAction::CreateSP(this, &SFoliagePalette::CanSelectInstances));

	UICommandList->MapAction(
		Commands.DeselectAllInstances,
		FExecuteAction::CreateSP(this, &SFoliagePalette::OnDeselectAllInstances),
		FCanExecuteAction::CreateSP(this, &SFoliagePalette::CanSelectInstances));

	UICommandList->MapAction(
		Commands.SelectInvalidInstances,
		FExecuteAction::CreateSP(this, &SFoliagePalette::OnSelectInvalidInstances),
		FCanExecuteAction::CreateSP(this, &SFoliagePalette::CanSelectInstances));
}

void SFoliagePalette::RefreshActivePaletteViewWidget()
{
	if (FoliageEditMode->UISettings.GetActivePaletteViewMode() == EFoliagePaletteViewMode::Thumbnail)
	{
		TileViewWidget->RequestListRefresh();
	}
	else
	{
		TreeViewWidget->RequestTreeRefresh();
	}
}

UFoliageType* SFoliagePalette::AddFoliageType(const FAssetData& AssetData, bool bPlaceholderAsset)
{
	if (AddFoliageTypeCombo.IsValid())
	{
		AddFoliageTypeCombo->SetIsMenuOpen(false, false);
	}

	GWarn->BeginSlowTask(LOCTEXT("AddFoliageType_LoadPackage", "Loading Foliage Type"), true, false);
	UObject* Asset = AssetData.GetAsset();
	GWarn->EndSlowTask();

	return FoliageEditMode->AddFoliageAsset(Asset, bPlaceholderAsset);
}

TSharedRef<SWidgetSwitcher> SFoliagePalette::CreatePaletteViews()
{
	const FText BlankText = FText::GetEmpty();

	// Tile View Widget
	SAssignNew(TileViewWidget, SFoliageTypeTileView)
		.ListItemsSource(&FilteredItems)
		.SelectionMode(ESelectionMode::Multi)
		.OnGenerateTile(this, &SFoliagePalette::GenerateTile)
		.OnContextMenuOpening(this, &SFoliagePalette::ConstructFoliageTypeContextMenu)
		.OnSelectionChanged(this, &SFoliagePalette::OnSelectionChanged)
		.ItemHeight(this, &SFoliagePalette::GetScaledThumbnailSize)
		.ItemWidth(this, &SFoliagePalette::GetScaledThumbnailSize)
		.ItemAlignment(EListItemAlignment::LeftAligned)
		.ClearSelectionOnClick(true)
		.OnMouseButtonDoubleClick(this, &SFoliagePalette::OnItemDoubleClicked);

	// Tree View Widget
	SAssignNew(TreeViewWidget, SFoliageTypeTreeView)
	.TreeItemsSource(&FilteredItems)
	.SelectionMode(ESelectionMode::Multi)
	.OnGenerateRow(this, &SFoliagePalette::TreeViewGenerateRow)
	.OnGetChildren(this, &SFoliagePalette::TreeViewGetChildren)
	.OnContextMenuOpening(this, &SFoliagePalette::ConstructFoliageTypeContextMenu)
	.OnSelectionChanged(this, &SFoliagePalette::OnSelectionChanged)
	.OnMouseButtonDoubleClick(this, &SFoliagePalette::OnItemDoubleClicked)
	.HeaderRow
	(
		// Toggle Active
		SAssignNew(TreeViewHeaderRow, SHeaderRow)
		+ SHeaderRow::Column(FoliagePaletteTreeColumns::ColumnID_ToggleActive)
		[
			SNew(SCheckBox)
			.IsChecked(this, &SFoliagePalette::GetState_AllMeshes)
			.OnCheckStateChanged(this, &SFoliagePalette::OnCheckStateChanged_AllMeshes)
		]
		.DefaultLabel(BlankText)
		.HeaderContentPadding(FMargin(0, 1, 0, 1))
		.HAlignHeader(HAlign_Center)
		.HAlignCell(HAlign_Center)
		.FixedWidth(24)

		// Type
		+ SHeaderRow::Column(FoliagePaletteTreeColumns::ColumnID_Type)
		.HeaderContentPadding(FMargin(10, 1, 0, 1))
		.SortMode(this, &SFoliagePalette::GetMeshColumnSortMode)
		.OnSort(this, &SFoliagePalette::OnMeshesColumnSortModeChanged)
		.DefaultLabel(this, &SFoliagePalette::GetMeshesHeaderText)
		.FillWidth(5.f)

		// Instance Count
		+ SHeaderRow::Column(FoliagePaletteTreeColumns::ColumnID_InstanceCount)
		.HeaderContentPadding(FMargin(10, 1, 0, 1))
		.DefaultLabel(LOCTEXT("InstanceCount", "Count"))
		.DefaultTooltip(this, &SFoliagePalette::GetTotalInstanceCountTooltipText)
		.FillWidth(2.f)

		// Save Asset
		+ SHeaderRow::Column(FoliagePaletteTreeColumns::ColumnID_Save)
		.FixedWidth(24.0f)
		.DefaultLabel(BlankText)
	);

	// View Mode Switcher
	SAssignNew(WidgetSwitcher, SWidgetSwitcher);

	// Thumbnail View
	WidgetSwitcher->AddSlot(EFoliagePaletteViewMode::Thumbnail)
	[
		SNew(SScrollBorder, TileViewWidget.ToSharedRef())
		.Content()
		[
			TileViewWidget.ToSharedRef()
		]
	];

	// Tree View
	WidgetSwitcher->AddSlot(EFoliagePaletteViewMode::Tree)
	[
		SNew(SScrollBorder, TreeViewWidget.ToSharedRef())
		.Style(&FAppStyle::Get().GetWidgetStyle<FScrollBorderStyle>("FoliageEditMode.TreeView.ScrollBorder"))
		.Content()
		[
			TreeViewWidget.ToSharedRef()
		]
	];

	WidgetSwitcher->SetActiveWidgetIndex(FoliageEditMode->UISettings.GetActivePaletteViewMode());

	return WidgetSwitcher.ToSharedRef();
}

void SFoliagePalette::GetPaletteItemFilterString(FFoliagePaletteItemModelPtr PaletteItemModel, TArray<FString>& OutArray) const
{
	OutArray.Add(PaletteItemModel->GetDisplayFName().ToString());
}

void SFoliagePalette::OnSearchTextChanged(const FText& InFilterText)
{
	TypeFilter->SetRawFilterText(InFilterText);
	SearchBoxPtr->SetError(TypeFilter->GetFilterErrorText());
	UpdatePalette();
}

void SFoliagePalette::AddFoliageTypePicker(const FAssetData& AssetData)
{
	AddFoliageType(AssetData);
}

TSharedRef<SWidget> SFoliagePalette::GetAddFoliageTypePicker()
{
	TArray<const UClass*> ClassFilters;

	FoliageEditMode->GetFoliageTypeFilters(ClassFilters);

	return PropertyCustomizationHelpers::MakeAssetPickerWithMenu(FAssetData(),
		false,
		ClassFilters,
		PropertyCustomizationHelpers::GetNewAssetFactoriesForClasses(ClassFilters),
		FOnShouldFilterAsset(),
		FOnAssetSelected::CreateSP(this, &SFoliagePalette::AddFoliageTypePicker),
		FSimpleDelegate());
}

void SFoliagePalette::HandleOnToolChanged()
{
	RefreshDetailsWidget();
}

void SFoliagePalette::SetViewMode(EFoliagePaletteViewMode::Type NewViewMode)
{
	EFoliagePaletteViewMode::Type ActiveViewMode = FoliageEditMode->UISettings.GetActivePaletteViewMode();
	if (ActiveViewMode != NewViewMode)
	{
		switch (NewViewMode)
		{
		case EFoliagePaletteViewMode::Thumbnail:
			// Set the tile selection to be the current tree selections
			TileViewWidget->ClearSelection();
			for (auto& TypeInfo : TreeViewWidget->GetSelectedItems())
			{
				TileViewWidget->SetItemSelection(TypeInfo, true);
			}
			break;
			
		case EFoliagePaletteViewMode::Tree:
			// Set the tree selection to be the current tile selection
			TreeViewWidget->ClearSelection();
			for (auto& TypeInfo : TileViewWidget->GetSelectedItems())
			{
				TreeViewWidget->SetItemSelection(TypeInfo, true);
			}
			break;
		}

		FoliageEditMode->UISettings.SetActivePaletteViewMode(NewViewMode);
		WidgetSwitcher->SetActiveWidgetIndex(NewViewMode);
		
		RefreshActivePaletteViewWidget();
	}
}

bool SFoliagePalette::IsActiveViewMode(EFoliagePaletteViewMode::Type ViewMode) const
{
	return FoliageEditMode->UISettings.GetActivePaletteViewMode() == ViewMode;
}

void SFoliagePalette::ToggleShowTooltips()
{
	const bool bCurrentlyShowingTooltips = FoliageEditMode->UISettings.GetShowPaletteItemTooltips();
	FoliageEditMode->UISettings.SetShowPaletteItemTooltips(!bCurrentlyShowingTooltips);
}

bool SFoliagePalette::ShouldShowTooltips() const
{
	return FoliageEditMode->UISettings.GetShowPaletteItemTooltips();
}

FText SFoliagePalette::GetSearchText() const
{
	return TypeFilter->GetRawFilterText();
}

void SFoliagePalette::OnSelectionChanged(FFoliagePaletteItemModelPtr Item, ESelectInfo::Type SelectInfo)
{
	RefreshDetailsWidget();

	bIsUneditableFoliageTypeSelected = false;
	for (FFoliagePaletteItemModelPtr& PaletteItem : GetActiveViewWidget()->GetSelectedItems())
	{
		// Currently entries from blueprint classes cannot be edited in the palette
		// as changes do not propagate to the BP class and changes to the BP class stomp any changes made to the instance in the palette item
		if (PaletteItem->IsBlueprint())
		{
			bIsUneditableFoliageTypeSelected = true;
			break;
		}
	}
}

void SFoliagePalette::OnItemDoubleClicked(FFoliagePaletteItemModelPtr Item) const
{
	Item->SetTypeActiveInPalette(!Item->IsActive());
}

TSharedRef<SWidget> SFoliagePalette::GetViewOptionsMenuContent()
{	
	const FFoliagePaletteCommands& Commands = FFoliagePaletteCommands::Get();
	FMenuBuilder MenuBuilder(true, UICommandList);

	MenuBuilder.BeginSection("FoliagePaletteViewMode", LOCTEXT("ViewModeHeading", "Palette View Mode"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ThumbnailView", "Thumbnails"),
			LOCTEXT("ThumbnailView_ToolTip", "Display thumbnails for each foliage type in the palette."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SFoliagePalette::SetViewMode, EFoliagePaletteViewMode::Thumbnail),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SFoliagePalette::IsActiveViewMode, EFoliagePaletteViewMode::Thumbnail)
				),
			NAME_None,
			EUserInterfaceActionType::RadioButton
			);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ListView", "List"),
			LOCTEXT("ListView_ToolTip", "Display foliage types in the palette as a list."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SFoliagePalette::SetViewMode, EFoliagePaletteViewMode::Tree),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SFoliagePalette::IsActiveViewMode, EFoliagePaletteViewMode::Tree)
				),
			NAME_None,
			EUserInterfaceActionType::RadioButton
			);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("FoliagePaletteViewOptions", LOCTEXT("ViewOptionsHeading", "View Options"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowTooltips", "Show Tooltips"),
			LOCTEXT("ShowTooltips_ToolTip", "Whether to show tooltips when hovering over foliage types in the palette."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SFoliagePalette::ToggleShowTooltips),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SFoliagePalette::ShouldShowTooltips)
				),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
			);

		MenuBuilder.AddWidget(
			SNew(SSlider)
				.ToolTipText( LOCTEXT("ThumbnailScaleToolTip", "Adjust the size of thumbnails.") )
				.Value( this, &SFoliagePalette::GetThumbnailScale)
				.OnValueChanged( this, &SFoliagePalette::SetThumbnailScale)
				.IsEnabled( this, &SFoliagePalette::GetThumbnailScaleSliderEnabled)
				.OnMouseCaptureEnd(this, &SFoliagePalette::RefreshActivePaletteViewWidget),
			LOCTEXT("ThumbnailScaleLabel", "Scale"),
			/*bNoIndent=*/true
			);

		//@todo: Hide inactive types
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedPtr<SListView<FFoliagePaletteItemModelPtr>> SFoliagePalette::GetActiveViewWidget() const
{
	const EFoliagePaletteViewMode::Type ActiveViewMode = FoliageEditMode->UISettings.GetActivePaletteViewMode();
	if (ActiveViewMode == EFoliagePaletteViewMode::Thumbnail)
	{
		return TileViewWidget;
	}
	else if (ActiveViewMode == EFoliagePaletteViewMode::Tree)
	{
		return TreeViewWidget;
	}
	
	return nullptr;
}

EVisibility SFoliagePalette::GetDropFoliageHintVisibility() const
{
	return FoliageEditMode->GetFoliageMeshList().Num() == 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SFoliagePalette::GetFoliageDropTargetVisibility() const
{
	if (FSlateApplication::Get().IsDragDropping())
	{
		TArray<const UClass*> ClassFilters;
		FoliageEditMode->GetFoliageTypeFilters(ClassFilters);
		// Support StaticMesh on drag
		ClassFilters.Add(UStaticMesh::StaticClass());

		TArray<FAssetData> DraggedAssets = AssetUtil::ExtractAssetDataFromDrag(FSlateApplication::Get().GetDragDroppingContent());
		for (const FAssetData& AssetData : DraggedAssets)
		{
			if (AssetData.IsValid())
			{
				for (const UClass* Filter : ClassFilters)
				{
					if (AssetData.IsInstanceOf(Filter))
					{
						return EVisibility::Visible;
					}
				}
			}
		}
	}

	return EVisibility::Hidden;
}

FReply SFoliagePalette::HandleFoliageDropped(const FGeometry& DropZoneGeometry, const FDragDropEvent& DragDropEvent)
{
	TArray<FAssetData> DroppedAssetData = AssetUtil::ExtractAssetDataFromDrag(DragDropEvent);
	if (DroppedAssetData.Num() > 0)
	{
		const bool bIsExternalContent = DragDropEvent.GetOperation().IsValid() && DragDropEvent.GetOperation()->IsOfType<FExternalContentDragDropOp>();

		// Treat the entire drop as a transaction (in case multiples types are being added)
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "FoliageMode_DragDropTypesTransaction", "Drag-drop Foliage"));

		TArray<TWeakObjectPtr<UFoliageType>> AddedFoliageTypes;

		for (auto& AssetData : DroppedAssetData)
		{
			if (UFoliageType* FoliageType = AddFoliageType(AssetData, bIsExternalContent))
			{
				AddedFoliageTypes.Add(FoliageType);
			}
		}

		if (AddedFoliageTypes.Num() > 0 && bIsExternalContent)
		{
			TSharedPtr<const FExternalContentDragDropOp> DragDropOp = StaticCastSharedPtr<const FExternalContentDragDropOp>(DragDropEvent.GetOperation());
			ExternalContentFoliageTypes.Add(DragDropOp->GetGuid(), AddedFoliageTypes);
		}
	}

	return FReply::Handled();
}

void SFoliagePalette::OnExternalContentResolved(const FGuid& Identifier, const FAssetData& PlaceHolderAsset, const FAssetData& ResolvedAsset)
{
	if (TArray<TWeakObjectPtr<UFoliageType>>* FoliageTypes = ExternalContentFoliageTypes.Find(Identifier))
	{
		TArray<UFoliageType*> ModifiedFoliageTypes;

		for (TWeakObjectPtr<UFoliageType> FoliageTypePtr : *FoliageTypes)
		{
			if (UFoliageType_InstancedStaticMesh* FoliageType = Cast<UFoliageType_InstancedStaticMesh>(FoliageTypePtr.Get()))
			{
				FAssetData AssetData(FoliageType->GetStaticMesh());
				if (AssetData == PlaceHolderAsset)
				{
					if (UStaticMesh* ResolvedMesh = Cast<UStaticMesh>(ResolvedAsset.GetAsset()))
					{
						FoliageType->Modify();
						FoliageType->SetStaticMesh(ResolvedMesh);
						ModifiedFoliageTypes.Add(FoliageType);
					}
				}
			}
		}
		ExternalContentFoliageTypes.Remove(Identifier);

		if (ModifiedFoliageTypes.Num() > 0)
		{
			for (TObjectIterator<AInstancedFoliageActor> It(RF_ClassDefaultObject, /** bIncludeDerivedClasses */ true, /** InternalExcludeFlags */ EInternalObjectFlags::Garbage); It; ++It)
			{
				if (It->GetWorld() != nullptr)
				{
					for (UFoliageType* ModifiedFoliageType : ModifiedFoliageTypes)
					{
						It->NotifyFoliageTypeChanged(ModifiedFoliageType, true);
					}
				}
			}
		}
	}
}

//	CONTEXT MENU

TSharedPtr<SWidget> SFoliagePalette::ConstructFoliageTypeContextMenu()
{
	const FFoliagePaletteCommands& Commands = FFoliagePaletteCommands::Get();
	FMenuBuilder MenuBuilder(true, UICommandList);

	auto SelectedItems = GetActiveViewWidget()->GetSelectedItems();
	
	if (SelectedItems.Num() > 0)
	{
		bool bSelectionContainsActorFoliage = SelectedItems.IndexOfByPredicate([](const FFoliagePaletteItemModelPtr& PaletteItem)
		{
			return Cast<const UFoliageType_Actor>(PaletteItem->GetFoliageType()) != nullptr;
		}) != INDEX_NONE;
		
		const bool bShowSaveAsOption = SelectedItems.Num() == 1 && !SelectedItems[0]->IsAsset() && !SelectedItems[0]->IsBlueprint();
		if (bShowSaveAsOption)
		{
			MenuBuilder.BeginSection("StaticMeshFoliageTypeOptions", LOCTEXT("StaticMeshFoliageTypeOptionsHeader", "Static Mesh"));
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("SaveAsFoliageType", "Save As Foliage Type..."),
					LOCTEXT("SaveAsFoliageType_ToolTip", "Creates a Foliage Type asset with these settings that can be reused in other levels."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Level.SaveIcon16x"),
					FUIAction(
						FExecuteAction::CreateSP(this, &SFoliagePalette::OnSaveSelected)
						),
					NAME_None
					);
			}
			MenuBuilder.EndSection();
		}

		MenuBuilder.BeginSection("FoliageTypeOptions", LOCTEXT("FoliageTypeOptionsHeader", "Foliage Type"));
		{
			if (!bShowSaveAsOption)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("SaveSelectedFoliageTypes", "Save"),
					LOCTEXT("SaveSelectedFoliageTypes_ToolTip", "Saves any changes to the selected foliage type asset(s)."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Level.SaveIcon16x"),
					FUIAction(
						FExecuteAction::CreateSP(this, &SFoliagePalette::OnSaveSelected),
						FCanExecuteAction::CreateSP(this, &SFoliagePalette::OnCanSaveAnySelectedAssets),
						FIsActionChecked(),
						FIsActionButtonVisible::CreateSP(this, &SFoliagePalette::GetIsPropertyEditingEnabled)
						),
					NAME_None
					);
			}

			MenuBuilder.AddMenuEntry(Commands.ActivateFoliageType);

			MenuBuilder.AddMenuEntry(Commands.DeactivateFoliageType);

			MenuBuilder.AddMenuEntry(Commands.RemoveFoliageType);

			MenuBuilder.AddSubMenu(
				LOCTEXT("ReplaceFoliageType", "Replace With..."),
				LOCTEXT("ReplaceFoliageType_ToolTip", "Replaces selected foliage type with another foliage type asset"),
				FNewMenuDelegate::CreateSP(this, &SFoliagePalette::FillReplaceFoliageTypeSubmenu));

			if (bSelectionContainsActorFoliage)
			{
				MenuBuilder.AddSubMenu(
					LOCTEXT("IncludeNonFoliageActors", "Include Non Foliage Actors..."),
					LOCTEXT("IncludeNonFoliageActors_ToolTip", "Include non foliage type actors with matching blueprint into foliage"),
					FNewMenuDelegate::CreateLambda([&](FMenuBuilder& SubMenuBuilder)
				{
					SubMenuBuilder.AddMenuEntry(LOCTEXT("IncludeNonFoliageActors_CurrentLevel", "Current Level"), FText(), FSlateIcon(), FExecuteAction::CreateSP(this, &SFoliagePalette::OnIncludeNonFoliageActors, true));
					SubMenuBuilder.AddMenuEntry(LOCTEXT("IncludeNonFoliageActors_AllLevels", "All Levels"), FText(), FSlateIcon(), FExecuteAction::CreateSP(this, &SFoliagePalette::OnIncludeNonFoliageActors, false));
				}));

				MenuBuilder.AddSubMenu(
					LOCTEXT("ExcludeFoliageActors", "Exclude Actors..."),
					LOCTEXT("ExcludeFoliageActors_ToolTip", "Exclude actors with foliage type from foliage"),
					FNewMenuDelegate::CreateLambda([&](FMenuBuilder& SubMenuBuilder)
				{
					SubMenuBuilder.AddMenuEntry(LOCTEXT("ExcludeFoliageActors_CurrentLevel", "Current Level"), FText(), FSlateIcon(), FExecuteAction::CreateSP(this, &SFoliagePalette::OnExcludeFoliageActors, true));
					SubMenuBuilder.AddMenuEntry(LOCTEXT("ExcludeFoliageActors_AllLevels", "All Levels"), FText(), FSlateIcon(), FExecuteAction::CreateSP(this, &SFoliagePalette::OnExcludeFoliageActors, false));
				}));
			}

			MenuBuilder.AddMenuEntry(Commands.ShowFoliageTypeInCB);
		}
		MenuBuilder.EndSection();
		
		MenuBuilder.BeginSection("InstanceSelectionOptions", LOCTEXT("InstanceSelectionOptionsHeader", "Selection"));
		{
			MenuBuilder.AddMenuEntry(Commands.SelectAllInstances);
			MenuBuilder.AddMenuEntry(Commands.DeselectAllInstances);
			MenuBuilder.AddMenuEntry(Commands.SelectInvalidInstances);
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

void SFoliagePalette::OnIncludeNonFoliageActors(bool bOnlyCurrentLevel)
{
	ExecuteOnSelectedItemFoliageTypes([&, bOnlyCurrentLevel](const TArray<const UFoliageType*>& FoliageTypes)
	{
		FoliageEditMode->IncludeNonFoliageActors(FoliageTypes, bOnlyCurrentLevel);
	});
}

void SFoliagePalette::OnExcludeFoliageActors(bool bOnlyCurrentLevel)
{
	ExecuteOnSelectedItemFoliageTypes([&, bOnlyCurrentLevel](const TArray<const UFoliageType*>& FoliageTypes)
	{
		FoliageEditMode->ExcludeFoliageActors(FoliageTypes, bOnlyCurrentLevel);
	});
}

void SFoliagePalette::OnSaveSelected()
{
	for (FFoliagePaletteItemModelPtr& PaletteItem : GetActiveViewWidget()->GetSelectedItems())
	{
		UFoliageType* FoliageType = PaletteItem->GetFoliageType();
		if (!FoliageType->IsAsset() || FoliageType->GetOutermost()->IsDirty())
		{
			UFoliageType* SavedFoliageType = FoliageEditMode->SaveFoliageTypeObject(FoliageType);
			if (SavedFoliageType)
			{
				FoliageType = SavedFoliageType;
			}
		}
	}
}

bool SFoliagePalette::OnCanSaveAnySelectedAssets() const
{
	// We can save if at least one of the selected items is a dirty asset
	for (FFoliagePaletteItemModelPtr& PaletteItem : GetActiveViewWidget()->GetSelectedItems())
	{
		UFoliageType* FoliageType = PaletteItem->GetFoliageType();
		if (FoliageType->IsAsset() && FoliageType->GetOutermost()->IsDirty())
		{
			return true;
		}
	}

	return false;
}

bool SFoliagePalette::AreAnyNonAssetTypesSelected() const
{
	for (FFoliagePaletteItemModelPtr& PaletteItem : GetActiveViewWidget()->GetSelectedItems())
	{
		if (!PaletteItem->GetFoliageType()->IsAsset())
		{
			// At least one selected type isn't an asset
			return true;
		}
	}

	return false;
}

void SFoliagePalette::OnActivateFoliageTypes()
{
	for (FFoliagePaletteItemModelPtr& PaletteItem : GetActiveViewWidget()->GetSelectedItems())
	{
		PaletteItem->SetTypeActiveInPalette(true);
	}
}

bool SFoliagePalette::OnCanActivateFoliageTypes() const
{
	// At least one selected item must be inactive
	for (FFoliagePaletteItemModelPtr& PaletteItem : GetActiveViewWidget()->GetSelectedItems())
	{
		if (!PaletteItem->IsActive())
		{
			return true;
		}
	}

	return false;
}

void SFoliagePalette::OnDeactivateFoliageTypes()
{
	for (FFoliagePaletteItemModelPtr& PaletteItem : GetActiveViewWidget()->GetSelectedItems())
	{
		PaletteItem->SetTypeActiveInPalette(false);
	}
}

bool SFoliagePalette::OnCanDeactivateFoliageTypes() const
{
	// At least one selected item must be active
	for (FFoliagePaletteItemModelPtr& PaletteItem : GetActiveViewWidget()->GetSelectedItems())
	{
		if (PaletteItem->IsActive())
		{
			return true;
		}
	}

	return false;
}

void SFoliagePalette::FillReplaceFoliageTypeSubmenu(FMenuBuilder& MenuBuilder)
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(UFoliageType::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SFoliagePalette::OnReplaceFoliageTypeSelected);
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.bAllowNullSelection = false;

	TSharedRef<SWidget> MenuContent = SNew(SBox)
		.WidthOverride(384.f)
		.HeightOverride(500.f)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];

	MenuBuilder.AddWidget(MenuContent, FText::GetEmpty(), true);
}

void SFoliagePalette::OnReplaceFoliageTypeSelected(const FAssetData& AssetData)
{
	FSlateApplication::Get().DismissAllMenus();

	UFoliageType* NewFoliageType = Cast<UFoliageType>(AssetData.GetAsset());
	if (GetActiveViewWidget()->GetSelectedItems().Num() && NewFoliageType)
	{
		for (FFoliagePaletteItemModelPtr& PaletteItem : GetActiveViewWidget()->GetSelectedItems())
		{
			UFoliageType* OldFoliageType = PaletteItem->GetFoliageType();
			if (OldFoliageType != NewFoliageType)
			{
				FoliageEditMode->ReplaceSettingsObject(OldFoliageType, NewFoliageType);
			}
		}
	}
}

void SFoliagePalette::OnRemoveFoliageType()
{
	int32 NumInstances = 0;
	TArray<UFoliageType*> FoliageTypeList;
	for (FFoliagePaletteItemModelPtr& PaletteItem : GetActiveViewWidget()->GetSelectedItems())
	{
		NumInstances += PaletteItem->GetTypeUIInfo()->InstanceCountTotal;
		FoliageTypeList.Add(PaletteItem->GetFoliageType());
	}

	bool bProceed = true;
	if (NumInstances > 0)
	{
		FText Message = FText::Format(NSLOCTEXT("UnrealEd", "FoliageMode_DeleteMesh", "Are you sure you want to remove {0} instances?"), FText::AsNumber(NumInstances));
		bProceed = (FMessageDialog::Open(EAppMsgType::YesNo, Message) == EAppReturnType::Yes);
	}

	if (bProceed)
	{
		FoliageEditMode->RemoveFoliageType(FoliageTypeList.GetData(), FoliageTypeList.Num());
	}
}

void SFoliagePalette::OnShowFoliageTypeInCB()
{
	TArray<UObject*> SelectedAssets;
	for (FFoliagePaletteItemModelPtr& PaletteItem : GetActiveViewWidget()->GetSelectedItems())
	{
		UFoliageType* FoliageType = PaletteItem->GetFoliageType();
		if (FoliageType->IsAsset())
		{
			SelectedAssets.Add(FoliageType);
		}
		else if (UBlueprint* FoliageTypeBP = Cast<UBlueprint>(FoliageType->GetClass()->ClassGeneratedBy))
		{
			SelectedAssets.Add(FoliageTypeBP);
		}
		else
		{
			SelectedAssets.Add(FoliageType->GetSource());
		}
	}

	if (SelectedAssets.Num())
	{
		GEditor->SyncBrowserToObjects(SelectedAssets);
	}
}

void SFoliagePalette::ReflectSelectionInPalette()
{
	TArray<const UFoliageType*> SelectedFoliageTypes;
	FoliageEditMode->GetSelectedInstanceFoliageTypes(SelectedFoliageTypes);
	SelectFoliageTypesInPalette(SelectedFoliageTypes);
}

void SFoliagePalette::SelectFoliageTypesInPalette(const TArray<const UFoliageType*>& FoliageTypes)
{
	TArray<FFoliagePaletteItemModelPtr> SelectedItems;
	SelectedItems.Reserve(FoliageTypes.Num());

	for (FFoliagePaletteItemModelPtr& PaletteItem : FilteredItems)
	{
		if (FoliageTypes.Contains(PaletteItem->GetFoliageType()))
		{
			SelectedItems.Add(PaletteItem);
		}
	}
	
	GetActiveViewWidget()->ClearSelection();
	GetActiveViewWidget()->SetItemSelection(SelectedItems, true);
}

void SFoliagePalette::ExecuteOnSelectedItemFoliageTypes(TFunctionRef<void(const TArray<const UFoliageType*>&)> ExecuteFunc)
{
	TArray<FFoliagePaletteItemModelPtr> SelectedItems = GetActiveViewWidget()->GetSelectedItems();
	TArray<const UFoliageType*> FoliageTypes;
	FoliageTypes.Reserve(SelectedItems.Num());
	for (FFoliagePaletteItemModelPtr& PaletteItem : SelectedItems)
	{
		FoliageTypes.Add(PaletteItem->GetFoliageType());
	}
	ExecuteFunc(FoliageTypes);
}

void SFoliagePalette::OnSelectAllInstances()
{
	ExecuteOnSelectedItemFoliageTypes([&](const TArray<const UFoliageType*>& FoliageTypes)
	{
		FoliageEditMode->SelectInstances(FoliageTypes, true);
	});	
}

void SFoliagePalette::OnDeselectAllInstances()
{
	ExecuteOnSelectedItemFoliageTypes([&](const TArray<const UFoliageType*>& FoliageTypes)
	{
		FoliageEditMode->SelectInstances(FoliageTypes, false);
	});
}

void SFoliagePalette::OnSelectInvalidInstances()
{
	ExecuteOnSelectedItemFoliageTypes([&](const TArray<const UFoliageType*>& FoliageTypes)
	{
		FoliageEditMode->SelectInvalidInstances(FoliageTypes);
	});
}

bool SFoliagePalette::CanSelectInstances() const
{
	return FoliageEditMode->UISettings.GetSelectToolSelected() || FoliageEditMode->UISettings.GetLassoSelectToolSelected();
}

// THUMBNAIL VIEW

TSharedRef<ITableRow> SFoliagePalette::GenerateTile(FFoliagePaletteItemModelPtr Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SFoliagePaletteItemTile, OwnerTable, Item);
}

float SFoliagePalette::GetScaledThumbnailSize() const
{
	const FInt32Interval& SizeRange = FoliagePaletteConstants::ThumbnailSizeRange;
	return SizeRange.Min + SizeRange.Size() * FoliageEditMode->UISettings.GetPaletteThumbnailScale();
}

float SFoliagePalette::GetThumbnailScale() const
{
	return FoliageEditMode->UISettings.GetPaletteThumbnailScale();
}

void SFoliagePalette::SetThumbnailScale(float InScale)
{
	FoliageEditMode->UISettings.SetPaletteThumbnailScale(InScale);
}

bool SFoliagePalette::GetThumbnailScaleSliderEnabled() const
{
	return FoliageEditMode->UISettings.GetActivePaletteViewMode() == EFoliagePaletteViewMode::Thumbnail;
}

// TREE VIEW

TSharedRef<ITableRow> SFoliagePalette::TreeViewGenerateRow(FFoliagePaletteItemModelPtr Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SFoliagePaletteItemRow, OwnerTable, Item);
}

void SFoliagePalette::TreeViewGetChildren(FFoliagePaletteItemModelPtr Item, TArray<FFoliagePaletteItemModelPtr>& OutChildren)
{
	//OutChildren = Item->GetChildren();
}

ECheckBoxState SFoliagePalette::GetState_AllMeshes() const
{
	bool bHasChecked = false;
	bool bHasUnchecked = false;

	for (const FFoliagePaletteItemModelPtr& PaletteItem : FilteredItems)
	{
		if (PaletteItem->IsActive())
		{
			bHasChecked = true;
		}
		else
		{
			bHasUnchecked = true;
		}

		if (bHasChecked && bHasUnchecked)
		{
			return ECheckBoxState::Undetermined;
		}
	}

	return bHasChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SFoliagePalette::OnCheckStateChanged_AllMeshes(ECheckBoxState InState)
{
	const bool bActivate = InState == ECheckBoxState::Checked;
	for (FFoliagePaletteItemModelPtr& PaletteItem : FilteredItems)
	{
		PaletteItem->SetTypeActiveInPalette(bActivate);
	}
}

FText SFoliagePalette::GetMeshesHeaderText() const
{
	int32 NumMeshes = FoliageEditMode->GetFoliageMeshList().Num();
	return FText::Format(LOCTEXT("FoliageMeshCount", "Meshes ({0})"), FText::AsNumber(NumMeshes));
}

EColumnSortMode::Type SFoliagePalette::GetMeshColumnSortMode() const
{
	return FoliageEditMode->GetFoliageMeshListSortMode();
}

void SFoliagePalette::OnMeshesColumnSortModeChanged(EColumnSortPriority::Type InPriority, const FName& InColumnName, EColumnSortMode::Type InSortMode)
{
	FoliageEditMode->OnFoliageMeshListSortModeChanged(InSortMode);
}

FText SFoliagePalette::GetTotalInstanceCountTooltipText() const
{
	// Probably should cache these values, 
	// but we call this only occasionally when tooltip is active
	int32 InstanceCountTotal = 0;
	int32 InstanceCountCurrentLevel = 0;
	FoliageEditMode->CalcTotalInstanceCount(InstanceCountTotal, InstanceCountCurrentLevel);

	return FText::Format(LOCTEXT("FoliageTotalInstanceCount", "Current Level: {0} Total: {1}"), FText::AsNumber(InstanceCountCurrentLevel), FText::AsNumber(InstanceCountTotal));
}

//	DETAILS VIEW

void SFoliagePalette::RefreshDetailsWidget()
{
	TArray<UObject*> SelectedFoliageTypes;

	for (const auto& PaletteItem : GetActiveViewWidget()->GetSelectedItems())
	{
		SelectedFoliageTypes.Add(PaletteItem->GetFoliageType());
	}

	const bool bForceRefresh = true;
	DetailsWidget->SetObjects(SelectedFoliageTypes, bForceRefresh);
}

bool SFoliagePalette::GetIsPropertyEditingEnabled() const
{
	return !bIsUneditableFoliageTypeSelected;
}

FText SFoliagePalette::GetDetailsNameAreaText() const
{
	FText OutText;

	auto SelectedItems = GetActiveViewWidget()->GetSelectedItems();
	if (SelectedItems.Num() == 1)
	{
		OutText = FText::FromName(SelectedItems[0]->GetDisplayFName());
	}
	else if (SelectedItems.Num() > 1)
	{
		OutText = FText::Format(LOCTEXT("DetailsNameAreaText_Multiple", "{0} Types Selected"), FText::AsNumber(SelectedItems.Num()));
	}

	return OutText;
}

FText SFoliagePalette::GetShowHideDetailsTooltipText() const
{
	const bool bDetailsCurrentlyVisible = DetailsWidget->GetVisibility() != EVisibility::Collapsed;
	return bDetailsCurrentlyVisible ? LOCTEXT("HideDetails_Tooltip", "Hide details for the selected foliage types.") : LOCTEXT("ShowDetails_Tooltip", "Show details for the selected foliage types.");
}

const FSlateBrush* SFoliagePalette::GetShowHideDetailsImage() const
{
	const bool bDetailsCurrentlyVisible = DetailsWidget->GetVisibility() != EVisibility::Collapsed;
	return FAppStyle::Get().GetBrush(bDetailsCurrentlyVisible ? "Symbols.DoubleDownArrow" : "Symbols.DoubleUpArrow");
}

void SFoliagePalette::OnShowHideDetailsClicked(const ECheckBoxState InCheckedState) const
{
	const bool bShouldShowDetails = InCheckedState == ECheckBoxState::Checked;

	DetailsWidget->SetVisibility(bShouldShowDetails ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed );
	FoliageEditMode->UISettings.SetShowPaletteItemDetails(bShouldShowDetails);
}

EVisibility SFoliagePalette::GetUneditableFoliageTypeWarningVisibility() const
{
	return bIsUneditableFoliageTypeSelected ? EVisibility::Visible : EVisibility::Collapsed;
}

void SFoliagePalette::OnEditFoliageTypeBlueprintHyperlinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	UBlueprint* Blueprint = nullptr; 
	
	// Get the first selected foliage type blueprint
	for (FFoliagePaletteItemModelPtr& PaletteItem : GetActiveViewWidget()->GetSelectedItems())
	{
		Blueprint = Cast<UBlueprint>(PaletteItem->GetFoliageType()->GetClass()->ClassGeneratedBy);
		if (Blueprint != nullptr)
		{
			break;
		}
	}

	if (Blueprint)
	{
		// Open the blueprint
		GEditor->EditObject(Blueprint);
	}
}

EActiveTimerReturnType SFoliagePalette::UpdatePaletteItems(double InCurrentTime, float InDeltaTime)
{
	if (bItemsNeedRebuild)
	{
		bItemsNeedRebuild = false;

		// Cache the currently selected items
		auto ActiveViewWidget = GetActiveViewWidget();
		TArray<FFoliagePaletteItemModelPtr> PreviouslySelectedItems = ActiveViewWidget->GetSelectedItems();
		ActiveViewWidget->ClearSelection();

		// Rebuild the list of palette items
		const auto& AllTypesList = FoliageEditMode->GetFoliageMeshList();
		PaletteItems.Empty(AllTypesList.Num());
		for (const FFoliageMeshUIInfoPtr& TypeInfo : AllTypesList)
		{
			PaletteItems.Add(MakeShareable(new FFoliagePaletteItemModel(TypeInfo, SharedThis(this), ThumbnailPool, FoliageEditMode)));
		}

		// Restore the selection
		for (auto& PrevSelectedItem : PreviouslySelectedItems)
		{
			// Select any replacements for previously selected foliage types
			for (auto& Item : PaletteItems)
			{
				if (Item->GetDisplayFName() == PrevSelectedItem->GetDisplayFName())
				{
					ActiveViewWidget->SetItemSelection(Item, true);
					break;
				}
			}
		}
	}

	// Update the filtered items
	FilteredItems.Empty();
	for (auto& Item : PaletteItems)
	{
		if (TypeFilter->PassesFilter(Item))
		{
			FilteredItems.Add(Item);
		}
	}

	// Refresh the appropriate view
	RefreshActivePaletteViewWidget();

	bIsRebuildTimerRegistered = false;
	return EActiveTimerReturnType::Stop;
}

#undef LOCTEXT_NAMESPACE
