// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "FoliageEdMode.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/Interval.h"
#include "Misc/Optional.h"
#include "Misc/TextFilter.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STileView.h"
#include "Widgets/Views/STreeView.h"

class FDragDropEvent;
class FFoliagePaletteItemModel;
class FMenuBuilder;
class FString;
class FUICommandList;
class ITableRow;
class SWidget;
class UFoliageType;
struct FAssetData;
struct FGeometry;
struct FKeyEvent;
struct FSlateBrush;
template <typename FuncType> class TFunctionRef;
template <typename ItemType> class SListView;

typedef TSharedPtr<FFoliagePaletteItemModel> FFoliagePaletteItemModelPtr;
typedef STreeView<FFoliagePaletteItemModelPtr> SFoliageTypeTreeView;
typedef STileView<FFoliagePaletteItemModelPtr> SFoliageTypeTileView;

namespace FoliagePaletteConstants
{
	const FInt32Interval ThumbnailSizeRange(32, 128);
}

/** The palette of foliage types available for use by the foliage edit mode */
class SFoliagePalette : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFoliagePalette) {}
		SLATE_ARGUMENT(FEdModeFoliage*, FoliageEdMode)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SFoliagePalette();

	// SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/** Updates the foliage palette, optionally doing a full rebuild of the items in the palette as well */
	void UpdatePalette(bool bRebuildItems = false);

	/** Updates the thumbnail for the given foliage type in the palette */
	void UpdateThumbnailForType(UFoliageType* FoliageType);

	bool AnySelectedTileHovered() const;
	void ActivateAllSelectedTypes(bool bActivate) const;

	/** @return True if the given view mode is the active view mode */
	bool IsActiveViewMode(EFoliagePaletteViewMode::Type ViewMode) const;

	/** @return True if tooltips should be shown when hovering over foliage type items in the palette */
	bool ShouldShowTooltips() const;

	/** @return The current search filter text */
	FText GetSearchText() const;

	/** Adds the foliage type asset to the instanced foliage actor's list of types. */
	UFoliageType* AddFoliageType(const FAssetData& AssetData, bool bPlaceholderAsset = false);

	/** Updates the selection of foliage types in the palette based on the selected foliage instances. */
	void ReflectSelectionInPalette();
	
private:	// GENERAL
	void AddFoliageTypePicker(const FAssetData& AssetData);

	/** Binds commands used by the palette */
	void BindCommands();

	/** Refreshes the active palette view widget */
	void RefreshActivePaletteViewWidget();

	/** Creates the palette views */
	TSharedRef<class SWidgetSwitcher> CreatePaletteViews();

	/** Adds the displayed name of the foliage type for filtering */
	void GetPaletteItemFilterString(FFoliagePaletteItemModelPtr PaletteItemModel, TArray<FString>& OutArray) const;

	/** Handles changes to the search filter text */
	void OnSearchTextChanged(const FText& InFilterText);

	/** Gets the asset picker for adding a foliage type. */
	TSharedRef<SWidget> GetAddFoliageTypePicker();

	/** Gets the visibility of the Add Foliage Type text in the header row button */
	EVisibility GetAddFoliageTypeButtonTextVisibility() const;

	/** Toggle whether all foliage types are active */
	ECheckBoxState GetState_AllMeshes() const;
	void OnCheckStateChanged_AllMeshes(ECheckBoxState InState);

	/** Handler to trigger a refresh of the details view when the active tool changes */
	void HandleOnToolChanged();

	void OnExternalContentResolved(const FGuid& Identifier, const FAssetData& PlaceholderAsset, const FAssetData& ResolvedAsset);

	/** Sets the view mode of the palette */
	void SetViewMode(EFoliagePaletteViewMode::Type NewViewMode);

	/** Sets whether to show tooltips when hovering over foliage type items in the palette */
	void ToggleShowTooltips();

	/** Switches the palette display between the tile and tree view */
	FReply OnToggleViewModeClicked();

	/** @return The index of the view widget to display */
	int32 GetActiveViewIndex() const;

	/** Handler for selection changes in either view */
	void OnSelectionChanged(FFoliagePaletteItemModelPtr Item, ESelectInfo::Type SelectInfo);

	/** Toggle the activation state of a type on a double-click */
	void OnItemDoubleClicked(FFoliagePaletteItemModelPtr Item) const;

	/** Creates the view options menu */
	TSharedRef<SWidget> GetViewOptionsMenuContent();

	TSharedPtr<SListView<FFoliagePaletteItemModelPtr>> GetActiveViewWidget() const;

	/** Gets the visibility of the "Drop Foliage Here" prompt for when the palette is empty */
	EVisibility GetDropFoliageHintVisibility() const;

	/** Gets the visibility of the drag-drop zone overlay */
	EVisibility GetFoliageDropTargetVisibility() const;

	/** Handles dropping of a mesh or foliage type into the palette */
	FReply HandleFoliageDropped(const FGeometry& DropZoneGeometry, const FDragDropEvent& DragDropEvent);

private:	// CONTEXT MENU

	/**  Finds actors with class matching foliage type and merges them into foliage instances */
	void OnIncludeNonFoliageActors(bool bOnlyCurrentLevel);

	/** Excludes actors from matching foliage type and make them regular actors. */
	void OnExcludeFoliageActors(bool bOnlyCurrentLevel);

	/** @return the SWidget containing the context menu */
	TSharedPtr<SWidget> ConstructFoliageTypeContextMenu();

	/** Saves a temporary non-asset foliage type (created from a static mesh) as a foliage type asset */
	void OnSaveSelected();

	/** @return True if all selected types are non-assets */
	bool OnCanSaveAnySelectedAssets() const;

	/** @return True if any of the selected types are non-assets */
	bool AreAnyNonAssetTypesSelected() const;

	/** Handler for the 'Activate' command */
	void OnActivateFoliageTypes();
	bool OnCanActivateFoliageTypes() const;

	/** Handler for the 'Deactivate' command */
	void OnDeactivateFoliageTypes();
	bool OnCanDeactivateFoliageTypes() const;

	/** Fills 'Replace' menu command  */
	void FillReplaceFoliageTypeSubmenu(FMenuBuilder& MenuBuilder);

	/** Handler for 'Replace' command  */
	void OnReplaceFoliageTypeSelected(const struct FAssetData& AssetData);

	/** Handler for 'Remove' command  */
	void OnRemoveFoliageType();

	/** Handler for 'Show in CB' command  */
	void OnShowFoliageTypeInCB();

	/** Handler for 'Select All' command  */
	void OnSelectAllInstances();

	/** Handler for 'Deselect All' command  */
	void OnDeselectAllInstances();

	/** Handler for 'Select Invalid Instances' command  */
	void OnSelectInvalidInstances();

	/** Executes Function on gathered list of foliage types from the selected palette items */
	void ExecuteOnSelectedItemFoliageTypes(TFunctionRef<void(const TArray<const UFoliageType*>&)> ExecuteFunc);

	/** @return Whether selecting instances is currently possible */
	bool CanSelectInstances() const;

	/** Selects Foliage Type in palette */
	void SelectFoliageTypesInPalette(const TArray<const UFoliageType*>& FoliageTypes);

private:	// THUMBNAIL VIEW

	/** Creates a thumbnail tile for the given foliage type */
	TSharedRef<ITableRow> GenerateTile(FFoliagePaletteItemModelPtr Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Gets the scaled thumbnail tile size */
	float GetScaledThumbnailSize() const;

	/** Gets the current scale of the thumbnail tiles */
	float GetThumbnailScale() const;

	/** Sets the current scale of the thumbnail tiles */
	void SetThumbnailScale(float InScale);

	/** Gets whether the thumbnail scaling slider is visible */
	bool GetThumbnailScaleSliderEnabled() const;

private:	// TREE VIEW

	/** Generates a row widget for foliage mesh item */
	TSharedRef<ITableRow> TreeViewGenerateRow(FFoliagePaletteItemModelPtr Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Generates a list of children items for foliage item */
	void TreeViewGetChildren(FFoliagePaletteItemModelPtr Item, TArray<FFoliagePaletteItemModelPtr>& OutChildren);

	/** Text for foliage meshes list header */
	FText GetMeshesHeaderText() const;

	/** Mesh list sorting support */
	EColumnSortMode::Type GetMeshColumnSortMode() const;
	void OnMeshesColumnSortModeChanged(EColumnSortPriority::Type InPriority, const FName& InColumnName, EColumnSortMode::Type InSortMode);

	/** Tooltip text for 'Instance Count" column */
	FText GetTotalInstanceCountTooltipText() const;

private:	// DETAILS

	/** Refreshes the mesh details widget to match the current selection */
	void RefreshDetailsWidget();

	/** Gets whether property editing is enabled */
	bool GetIsPropertyEditingEnabled() const;

	/** Gets the text for the details area header */
	FText GetDetailsNameAreaText() const;

	/** Gets the text for the show/hide details button tooltip */
	FText GetShowHideDetailsTooltipText() const;

	/** Gets the image for the show/hide details button */
	const FSlateBrush* GetShowHideDetailsImage() const;

	/** Handles the show/hide details button click */
	void OnShowHideDetailsClicked(const ECheckBoxState ) const;

	/** Gets the visibility of the uneditable blueprint foliage type warning */
	EVisibility GetUneditableFoliageTypeWarningVisibility() const;

	/** Handles the click for the uneditable blueprint foliage type warning */
	void OnEditFoliageTypeBlueprintHyperlinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata);

private:
	/** Active timer handler to update the items in the palette */
	EActiveTimerReturnType UpdatePaletteItems(double InCurrentTime, float InDeltaTime);

private:
	typedef TTextFilter<FFoliagePaletteItemModelPtr> FoliageTypeTextFilter;
	TSharedPtr<FoliageTypeTextFilter> TypeFilter;

	/** All the items in the palette (unfiltered) */
	TArray<FFoliagePaletteItemModelPtr> PaletteItems;

	/** The filtered list of types to display in the palette */
	TArray<FFoliagePaletteItemModelPtr> FilteredItems;

	/** Switches between the thumbnail and tree views */
	TSharedPtr<class SWidgetSwitcher> WidgetSwitcher;

	/** The Add Foliage Type combo button */
	TSharedPtr<class SPositiveActionButton> AddFoliageTypeCombo;

	/** The header row of the foliage mesh tree */
	TSharedPtr<class SHeaderRow> TreeViewHeaderRow;

	/** Foliage type thumbnails widget  */
	TSharedPtr<SFoliageTypeTileView> TileViewWidget;

	/** Foliage type tree widget  */
	TSharedPtr<SFoliageTypeTreeView> TreeViewWidget;

	/** Foliage mesh details widget  */
	TSharedPtr<class IDetailsView> DetailsWidget;

	/** Foliage items search box widget */
	TSharedPtr<class SSearchBox> SearchBoxPtr;

	/** Command list for binding functions for the context menu. */
	TSharedPtr<FUICommandList> UICommandList;

	/** Thumbnail pool for rendering mesh thumbnails */
	TSharedPtr<class FAssetThumbnailPool> ThumbnailPool;

	FEdModeFoliage* FoliageEditMode;

	TMap<FGuid, TArray<TWeakObjectPtr<UFoliageType>>> ExternalContentFoliageTypes;

	bool bItemsNeedRebuild : 1;
	bool bIsUneditableFoliageTypeSelected : 1;
	
	bool bIsRebuildTimerRegistered : 1;
};
