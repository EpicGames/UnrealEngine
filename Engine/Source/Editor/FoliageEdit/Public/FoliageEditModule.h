// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "UObject/NameTypes.h"
#include "FoliageModule.h"

extern const FName FoliageEditAppIdentifier;

class ULevel;

/**
 * Foliage Edit mode module interface
 */
class IFoliageEditModule : public IFoliageEditModuleBase
{
public:
};
