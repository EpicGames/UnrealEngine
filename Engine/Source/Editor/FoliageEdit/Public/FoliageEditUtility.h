// Copyright Epic Games, Inc. All Rights Reserved.

/**
* Utility used for Foliage Edition
*/

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

class AActor;
class UFoliageType;
class ULevel;
class UWorld;
struct FFoliageMeshUIInfo;

class FFoliageEditUtility
{
public:
	/** Save the foliage type object. If it isn't an asset, will prompt the user for a location to save the new asset. */
	static FOLIAGEEDIT_API UFoliageType* SaveFoliageTypeObject(UFoliageType* Settings, bool bPlaceholderAsset = false); 

	/**
	 * Copies the provided FoliageType to a new asset
	 * @param 	InPackageName 	The name of the new asset
	 * @param 	InFoliageType 	The FoliageType to copy
	 * @return 	The new FoliageType
	 */
	static FOLIAGEEDIT_API UFoliageType* DuplicateFoliageTypeToNewPackage(const FString& InPackageName, UFoliageType* InFoliageType);

	static FOLIAGEEDIT_API void ReplaceFoliageTypeObject(UWorld* InWorld, UFoliageType* OldType, UFoliageType* NewType);

	static FOLIAGEEDIT_API void MoveActorFoliageInstancesToLevel(ULevel* InTargetLevel, AActor* InIFA = nullptr);
};