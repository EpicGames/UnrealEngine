// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "Engine/World.h"
#include "FoliageEditorSubsystem.generated.h"

class ULevel;
class AActor;

UCLASS()
class UFoliageEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

	UFoliageEditorSubsystem();
public:
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

private:
	void OnActorMoved(AActor* InActor);
	void OnActorOuterChanged(AActor* InActor, UObject* OldOuter);
	void OnActorDeleted(AActor* InActor);
	void OnPostApplyLevelOffset(ULevel* InLevel, UWorld* InWorld, const FVector& InOffset, bool bWorldShift);
	void OnPostApplyLevelTransform(ULevel* InLevel, const FTransform& InTransform);
	void OnPostWorldInitialization(UWorld* InWorld, const UWorld::InitializationValues IVS);
};
