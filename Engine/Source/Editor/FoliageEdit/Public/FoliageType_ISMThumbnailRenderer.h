// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * This thumbnail renderer displays the static mesh used by this foliage type
 */

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "FoliageType_ISMThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;
class UObject;

UCLASS(CustomConstructor, Config=Editor)
class UFoliageType_ISMThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()
	
	UFoliageType_ISMThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	:	Super(ObjectInitializer)
	,	ThumbnailScene(nullptr)
	{}

	// UThumbnailRenderer implementation
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	virtual bool CanVisualizeAsset(UObject* Object) override;
	// UObject implementation
	virtual void BeginDestroy() override;

private:
	class FStaticMeshThumbnailScene* ThumbnailScene;
};
