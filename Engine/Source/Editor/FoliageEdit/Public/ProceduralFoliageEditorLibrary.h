// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 */

#pragma once

#include "Containers/Array.h"
#include "Engine/EngineTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ProceduralFoliageEditorLibrary.generated.h"

class AProceduralFoliageVolume;
class UObject;
class UProceduralFoliageComponent;
struct FFrame;

UCLASS()
class UProceduralFoliageEditorLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Foliage")
	static void ResimulateProceduralFoliageVolumes(const TArray<AProceduralFoliageVolume*>& ProceduralFoliageVolumes);

	UFUNCTION(BlueprintCallable, Category="Foliage")
	static void ResimulateProceduralFoliageComponents(const TArray<UProceduralFoliageComponent*>& ProceduralFoliageComponents);

	UFUNCTION(BlueprintCallable, Category="Foliage")
	static void ClearProceduralFoliageVolumes(const TArray<AProceduralFoliageVolume*>& ProceduralFoliageVolumes);

	UFUNCTION(BlueprintCallable, Category = "Foliage")
	static void ClearProceduralFoliageComponents(const TArray<UProceduralFoliageComponent*>& ProceduralFoliageComponents);
};
